{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Apple ProRes Media",
	"Description": "Implements video playback and the export of the Apple ProRes Codec.  Apple ProRes is a high quality, lossy video compression format.",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"Category": "Media Players",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [ "Win64" ],
	"Modules": [
		{
			"Name": "AppleProResMedia",
			"Type": "Runtime",
			"LoadingPhase": "PostEngineInit",
			"PlatformAllowList": [ "Win64", "Mac" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ],
			"TargetDenyList": [ "Server" ]
		}
	],
	"Plugins": [
		{
			"Name": "WmfMedia",
			"Enabled": true
		},
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true
		}
	]
}
