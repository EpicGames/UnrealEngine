// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class AppleProResMedia : ModuleRules
	{
		public AppleProResMedia(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
            PublicDependencyModuleNames.AddRange(
				new string[] {
                    "ProResLib",
					"ProResToolbox",
                    "WmfMediaFactory",
					"ImageWriteQueue",
					"MovieRenderPipelineCore",
					"OpenColorIO",
					"SlateCore",
                });

			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
                    "CoreUObject",
                    "Engine",
                    "MovieSceneCapture",
                    "Projects",
                    "WmfMediaFactory"
                }
            );

            if (Target.Platform == UnrealTargetPlatform.Win64)
            {
				PublicDependencyModuleNames.Add("WmfMedia");
				PrivateDependencyModuleNames.Add("WmfMedia");
				
                PublicDelayLoadDLLs.Add("mf.dll");
                PublicDelayLoadDLLs.Add("mfplat.dll");
                PublicDelayLoadDLLs.Add("mfplay.dll");
                PublicDelayLoadDLLs.Add("shlwapi.dll");

                PublicSystemLibraries.Add("mf.lib");
                PublicSystemLibraries.Add("mfplat.lib");
                PublicSystemLibraries.Add("mfuuid.lib");
                PublicSystemLibraries.Add("shlwapi.lib");
                PublicSystemLibraries.Add("d3d11.lib");
            }
			else if (Target.Platform == UnrealTargetPlatform.Mac)
			{
				PublicFrameworks.Add("VideoToolbox");
			}
        }
    }
}
