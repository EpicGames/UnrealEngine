// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleProResEncoder/AppleProResEncoder.h"
#include "AppleProResMediaModule.h"
#include "HAL/FileManager.h"
#include "Misc/ScopeExit.h"
#include "ProResToolboxExtension.h"

namespace AppleProRes
{
	static int32_t GetFrameHeaderColorPrimaries(EAppleProResEncoderColorPrimaries InColorPrimary)
	{
		switch (InColorPrimary)
		{
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_525_60HZ:
			return kProResFormatDescriptionColorPrimaries_SMPTE_C;
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_625_50HZ:
			return kProResFormatDescriptionColorPrimaries_EBU_3213;
		case EAppleProResEncoderColorPrimaries::CD_HDREC709:
			return kProResFormatDescriptionColorPrimaries_ITU_R_709_2;
		default:
			check(false);
		}

		return 0;
	}

	static ProResFormatDescriptionColorPrimaries GetColorPrimaries(EAppleProResEncoderColorPrimaries InColorPrimary)
	{
		switch (InColorPrimary)
		{
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_525_60HZ:
			return kProResFormatDescriptionColorPrimaries_SMPTE_C;
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_625_50HZ:
			return kProResFormatDescriptionColorPrimaries_EBU_3213;
		case EAppleProResEncoderColorPrimaries::CD_HDREC709:
			return kProResFormatDescriptionColorPrimaries_ITU_R_709_2;
		default:
			check(false);
		}

		return kProResFormatDescriptionColorPrimaries_SMPTE_C;
	}

	static ProResFormatDescriptionTransferFunction GetTransferFunction(EAppleProResEncoderColorPrimaries InColorPrimary)
	{
		switch (InColorPrimary)
		{
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_525_60HZ:
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_625_50HZ:
		case EAppleProResEncoderColorPrimaries::CD_HDREC709:
			return kProResFormatDescriptionTransferFunction_ITU_R_709_2;
		default:
			check(false);
		}

		return kProResFormatDescriptionTransferFunction_ITU_R_709_2;
	}

	static int32_t GetFrameHeaderTransferCharacteristic(EAppleProResEncoderColorPrimaries InColorPrimary)
	{
		switch (InColorPrimary)
		{
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_525_60HZ:
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_625_50HZ:
		case EAppleProResEncoderColorPrimaries::CD_HDREC709:
			return kPRTransferCharacteristic_ITU_R_709;
		default:
			check(false);
		}

		return 0;
	}

	static ProResFormatDescriptionYCbCrMatrix GetYCbCrMatrix(EAppleProResEncoderColorPrimaries InColorPrimary)
	{
		switch (InColorPrimary)
		{
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_525_60HZ:
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_625_50HZ:
			return kProResFormatDescriptionYCbCrMatrix_ITU_R_601_4;
		case EAppleProResEncoderColorPrimaries::CD_HDREC709:
			return kProResFormatDescriptionYCbCrMatrix_ITU_R_709_2;
		default:
			check(false);
		}

		return kProResFormatDescriptionYCbCrMatrix_ITU_R_601_4;
	}

	static int32_t GetFrameHeaderMatrixCoefficients(EAppleProResEncoderColorPrimaries InColorPrimary)
	{
		switch (InColorPrimary)
		{
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_525_60HZ:
		case EAppleProResEncoderColorPrimaries::CD_SDREC601_625_50HZ:
			return kPRMatrixCoefficients_ITU_R_601;
		case EAppleProResEncoderColorPrimaries::CD_HDREC709:
			return kPRMatrixCoefficients_ITU_R_709;
		default:
			check(false);
		}

		return 0;
	}

	static PRInterlaceMode GetInterlaceMode(EAppleProResEncoderScanMode InScanMode)
	{
		switch (InScanMode)
		{
		case EAppleProResEncoderScanMode::IM_PROGRESSIVE_SCAN:
			return kPRProgressiveScan;
		case EAppleProResEncoderScanMode::IM_INTERLACED_TOP_FIELD_FIRST:
			return kPRInterlacedTopFieldFirst;
		case EAppleProResEncoderScanMode::IM_INTERLATED_BOTTOM_FIRST_FIRST:
			return kPRInterlacedBottomFieldFirst;
		default:
			check(false);
		}

		return kPRProgressiveScan;
	}

	static uint32_t GetFieldCount(EAppleProResEncoderScanMode InScanMode)
	{
		switch (InScanMode)
		{
		case EAppleProResEncoderScanMode::IM_PROGRESSIVE_SCAN:
			return 1;
		case EAppleProResEncoderScanMode::IM_INTERLACED_TOP_FIELD_FIRST:
		case EAppleProResEncoderScanMode::IM_INTERLATED_BOTTOM_FIRST_FIRST:
			return 2;
		default:
			check(false);
		}

		return 0;
	}

	static uint32_t GetBitDepth(EAppleProResEncoderCodec InEncodingFormat, bool bWriteAlpha)
	{
		switch (InEncodingFormat)
		{
		case EAppleProResEncoderCodec::ProRes_4444:
			return bWriteAlpha ? 32 : 24;
		default:
			return 24;
		}
	}

	static PRCodecType GetCodecType(EAppleProResEncoderCodec InEncodingFormat)
	{
		switch (InEncodingFormat)
		{
		case EAppleProResEncoderCodec::ProRes_422HQ:
			return kPRType422HQ;
		case EAppleProResEncoderCodec::ProRes_422:
			return kPRType422;
		case EAppleProResEncoderCodec::ProRes_422LT:
			return kPRType422LT;
		case EAppleProResEncoderCodec::ProRes_422Proxy:
			return kPRType422Proxy;
		case EAppleProResEncoderCodec::ProRes_4444:
			return kPRType4444;
		case EAppleProResEncoderCodec::ProRes_4444XQ:
			return kPRType4444XQ;
		default:
			check(false);
		}

		return kPRType422HQ;
	}

	static ProResFormatDescriptionFieldDetail GetFieldDetail(EAppleProResEncoderScanMode InScanMode)
	{
		switch (InScanMode)
		{
		case EAppleProResEncoderScanMode::IM_PROGRESSIVE_SCAN:
			return kProResFormatDescriptionFieldDetail_Unknown;
		case EAppleProResEncoderScanMode::IM_INTERLACED_TOP_FIELD_FIRST:
			return kProResFormatDescriptionFieldDetail_SpatialFirstLineEarly;
		case EAppleProResEncoderScanMode::IM_INTERLATED_BOTTOM_FIRST_FIRST:
			return kProResFormatDescriptionFieldDetail_SpatialFirstLineLate;
		default:
			check(false);
		}

		return kProResFormatDescriptionFieldDetail_Unknown;
	}

	static bool ConvertPixelDataToRGBA4444(const FImagePixelData* InPixelData, bool bWriteAlpha, TArray<uint8>& OutDestinationBuffer)
	{
		switch (InPixelData->GetType())
		{
		case EImagePixelType::Color:
		{
			// We upscale 8 bit to 16 bit ProRes will encode it down to 4:4:4 or similar based on codec anyways.
			int64 SizeInBytes;
			const void* OutRawData = nullptr;
			InPixelData->GetRawData(OutRawData, SizeInBytes);
			int64 NumPixels = InPixelData->GetSize().X * InPixelData->GetSize().Y;

			uint8* SourceBuffer = (uint8*)OutRawData;
			uint8* DestinationBuffer = (uint8*)OutDestinationBuffer.GetData();
			const uint8 AlphaMultiplier = bWriteAlpha ? 1 : 0;

			// Output takes 16 bit, we write into the high order bits of each int16.
			for (int64 Index = 0; Index < NumPixels; Index++)
			{
				DestinationBuffer[0] = SourceBuffer[3] * AlphaMultiplier;	// A
				DestinationBuffer[2] = SourceBuffer[2];	// R
				DestinationBuffer[4] = SourceBuffer[1]; // G
				DestinationBuffer[6] = SourceBuffer[0]; // B
				DestinationBuffer += 8;
				SourceBuffer += 4;
			}
		}
		return true;
		case EImagePixelType::Float16:
		{
			// Assumes our data is in the [0-1] range (Tonecurve applied, sRGB applied) multiply by 65535 to fill out all of the bits.
			int64 SizeInBytes;
			const void* OutRawData = nullptr;
			InPixelData->GetRawData(OutRawData, SizeInBytes);
			int64 NumPixels = InPixelData->GetSize().X * InPixelData->GetSize().Y;

			const FFloat16Color* ColorData = (FFloat16Color*)(OutRawData);

			uint8* DestinationBuffer = (uint8*)OutDestinationBuffer.GetData();
			const uint8 AlphaMultiplier = bWriteAlpha ? 1 : 0;

			// Output takes 16 bit, we write into the high order bits of each int16.
			for (int64 Index = 0; Index < NumPixels; Index++)
			{
				uint16 A = FMath::Clamp((int32)((ColorData[Index].A) * AlphaMultiplier * 65535), 0, 65535);
				uint16 R = FMath::Clamp((int32)(ColorData[Index].R * 65535), 0, 65535);
				uint16 G = FMath::Clamp((int32)(ColorData[Index].G * 65535), 0, 65535);
				uint16 B = FMath::Clamp((int32)(ColorData[Index].B * 65535), 0, 65535);

				DestinationBuffer[0] = A >> 8;
				DestinationBuffer[1] = A & 0x00FF;
				DestinationBuffer[2] = R >> 8;
				DestinationBuffer[3] = R & 0x00FF;
				DestinationBuffer[4] = G >> 8;
				DestinationBuffer[5] = G & 0x00FF;
				DestinationBuffer[6] = B >> 8;
				DestinationBuffer[7] = B & 0x00FF;
				DestinationBuffer += 8;
			}
		}
		return true;
		}

		return false;

	}
}

FAppleProResEncoder::FAppleProResEncoder(const FAppleProResEncoderOptions& InOptions)
    : Options(InOptions)
	, bInitialized(false)
	, bFinalized(false)
	, VideoFormatDescription(NULL)
	, AudioFormatDescription(NULL)
	, TimecodeFormatDescription(NULL)
	, FileWriter(NULL)
	, Encoder(NULL)
{
     
}

FAppleProResEncoder::~FAppleProResEncoder()
{
    // Insure Finalize is called so that we release resources if we were ever initialized.
    Finalize();
}

bool FAppleProResEncoder::Initialize()
{
	// Manually loading the toolbox library, on demand only.
	if (!UE::AppleProResMedia::LoadProResToolbox())
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to initialize ProRes Encoder.");
		return false;
	}
	
	// Attempt to create the Encoder
	Encoder = PROpenEncoder(FMath::Max(Options.MaxNumberOfEncodingThreads, 0), /*ThreadStartupCallback*/ nullptr);
	if (!Encoder)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to initialize ProRes Encoder.");
		return false;
	}

	ON_SCOPE_EXIT
	{
		// Clear any allocated resources if initialization failed.
		if (!bInitialized)
		{
			ReleaseAndClearResources();
		}
	};

	// Delete the file on disk if it already exists as ProRes doesn't handle that for us.
	if (IFileManager::Get().FileExists(*Options.OutputFilename))
	{
		UE_LOGF(LogAppleProResMedia, Display, "Deleting existing file \"%ls\" (%lld bytes) before initializing new one.", *Options.OutputFilename, IFileManager::Get().FileSize(*Options.OutputFilename));
		IFileManager::Get().Delete(*Options.OutputFilename);
	}

	// Create the file. Need to convert the supplied filepath to ANSICHAR
	PRStatus Status = ProResFileWriterCreate(TCHAR_TO_ANSI(*Options.OutputFilename), /*Out*/ &FileWriter);
	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to create file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

	// ProRes does Numerator/Denominator like us but reversed - Our time scale is frames-per-second, they are seconds-per-frame so we swap math.
	PRTimeScale ProResTimescale = Options.FrameRate.Numerator;

	CurrentVideoTime.value = Options.FrameNumberOffset * Options.FrameRate.Denominator;
	CurrentVideoTime.timescale = Options.FrameRate.Numerator;
	CurrentVideoTime.flags = kPRTimeFlags_Valid;
	CurrentVideoTime.epoch = 0;

	// Set the time scale in the header. Must match the samples we write later and must be before header is written.
	Status = ProResFileWriterSetMovieTimescale(FileWriter, ProResTimescale);
	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to set Movie Timescale");
		return false;
	}

	// Add our Video stream
	{
		Status = ProResFileWriterAddTrack(FileWriter, kPRMediaType_Video, &VideoTrackId);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to add video track to file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}
		Status = ProResFileWriterSetTrackMediaTimescale(FileWriter, VideoTrackId, ProResTimescale);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to set video track timescale to file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}
	}
	
	// Add a Audio stream
	{
		if (Options.bIncludeAudio)
		{
			Status = ProResFileWriterAddTrack(FileWriter, kPRMediaType_Audio, &AudioTrackId);
			if (Status != 0)
			{
				UE_LOGF(LogAppleProResMedia, Error, "Failed to add audio track to file writer at path: \"%ls\".", *Options.OutputFilename);
				return false;
			}
		
			// Sound tracks must match the sample rate, not frame rate.
			if(!ensureMsgf(Options.AudioSampleRate > 0, TEXT("AudioSampleRate must be greater than zero!")))
			{
				return false;
			}
		
			PRTimeScale ProResAudioTimescale = Options.AudioSampleRate;
			Status = ProResFileWriterSetTrackMediaTimescale(FileWriter, AudioTrackId, ProResAudioTimescale);
			if (Status != 0)
			{
				UE_LOGF(LogAppleProResMedia, Error, "Failed to set audio track timescale to file writer at path: \"%ls\".", *Options.OutputFilename);
				return false;
			}
		}
	}

	// Add a Timecode stream
	{
		Status = ProResFileWriterAddTrack(FileWriter, kPRMediaType_Timecode, &TimecodeTrackId);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to add timecode track to file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}
		Status = ProResFileWriterSetTrackMediaTimescale(FileWriter, TimecodeTrackId, ProResTimescale);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to set timecode track timescale to file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}
	}

	// Set up the Video Track 
	{
		bool bSuccess = InitializeVideoTrack();
		if (!bSuccess)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to initialize video track with specified settings for file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}

		bSuccess = InitializeTimecodeTrack();
		if (!bSuccess)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to initialize timecode track with specified settings for file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}

		if (Options.bIncludeAudio)
		{
			bSuccess = InitializeAudioTrack();
			if (!bSuccess)
			{
				UE_LOGF(LogAppleProResMedia, Error, "Failed to initialize audio track with specified settings for file writer at path: \"%ls\".", *Options.OutputFilename);
				return false;
			}
		}
	}

	Status = ProResFileWriterBeginSession(FileWriter, CurrentVideoTime);
	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to begin session with file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

    bInitialized = true;
	return true;
}



bool FAppleProResEncoder::InitializeVideoTrack()
{
	// Calculate how big a given sample is for their provided settings. This lets us know how big of a buffer to make.
	PRGetCompressedFrameSize(AppleProRes::GetCodecType(Options.Codec), Options.bWriteAlpha, Options.Width, Options.Height, &MaxCompressedFrameSize, &TargetCompressedFrameSize);
	
	// A reccomended chunk size for video codecs is 4MiB for HD codecs.
	const int32 MaxChunkSize = 4 * 1024 * 1024;
	const int32 MaxFrameCount = FMath::Max(MaxChunkSize / TargetCompressedFrameSize, 1); // Figure out how many frames per chunk will fit into 4MiB

	PRStatus Status = ProResFileWriterSetTrackPreferredChunkSize(FileWriter, VideoTrackId, MaxFrameCount * TargetCompressedFrameSize);
	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to set preferred chunk size for video track for file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

	const ProResFormatDescriptionColorPrimaries ColorPrimaries = AppleProRes::GetColorPrimaries(Options.ColorPrimaries);
	const ProResFormatDescriptionTransferFunction TransferFunction = AppleProRes::GetTransferFunction(Options.ColorPrimaries);
	const ProResFormatDescriptionYCbCrMatrix Matrix = AppleProRes::GetYCbCrMatrix(Options.ColorPrimaries);

	const uint32 PixelAspectRatio = 1;
	PRVideoDimensions Dimensions;
	Dimensions.width = Options.Width;
	Dimensions.height = Options.Height;

	// Create a Video Format Description. Caller owns the returned FormatDescription.
	Status = ProResVideoFormatDescriptionCreate(
		AppleProRes::GetCodecType(Options.Codec),
		Dimensions,
		AppleProRes::GetBitDepth(Options.Codec, Options.bWriteAlpha),
		AppleProRes::GetFieldCount(Options.ScanMode),
		AppleProRes::GetFieldDetail(Options.ScanMode),
		ColorPrimaries,
		TransferFunction,
		Matrix,
		PixelAspectRatio,
		PixelAspectRatio,
		nullptr,
		false,
		0.0f,
		&VideoFormatDescription);

	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to create a VideoFormatDescription from settings for file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

	return true;
}

bool FAppleProResEncoder::InitializeTimecodeTrack()
{
	PRTime FrameDuration;
	FrameDuration.value = Options.FrameRate.Denominator;
	FrameDuration.timescale = Options.FrameRate.Numerator;
	FrameDuration.flags = kPRTimeFlags_Valid;
	FrameDuration.epoch = 0;

	const FFrameRate TwentyNineNineSeven = FFrameRate(30000, 1001);

	uint32_t FrameQuanta = FMath::CeilToInt((float)Options.FrameRate.Numerator / (float)Options.FrameRate.Denominator);
	PRStatus Status = ProResTimecodeFormatDescriptionCreate(
		FrameDuration,
		FrameQuanta,
		(Options.bDropFrameTimecode && (Options.FrameRate == TwentyNineNineSeven)) ? kProResTimecodeFlag_DropFrame : 0,
		nullptr,
		0,
		0,
		&TimecodeFormatDescription);
	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to create a TimecodeFormatDescription from settings for file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

	return true;
}

bool FAppleProResEncoder::InitializeAudioTrack()
{
	AudioStreamBasicDescription AudioSettings = {};
	AudioSettings.mSampleRate = Options.AudioSampleRate; 
	AudioSettings.mFormatID = kAudioFormatLinearPCM;
	AudioSettings.mFormatFlags = kAudioFormatFlagIsSignedInteger | kAudioFormatFlagIsPacked;

	// To figure out how much audio data there is per sample, we take our AudioSampleRate (48000) and divide it by our framerate, ie: 48'000 / 30 = 1600
	AudioSamplesPerFrame = FMath::FloorToInt(Options.AudioSampleRate * Options.FrameRate.AsInterval());

	// Audio is uncompressed lpcm, and for Apple ProRes 1 frame is 1 sample across all channels, and 1 packet is 1 frame in compressed files.
	// Then when it's submitted, we can submit one video-frame's worth of data at a time.
	AudioSettings.mBytesPerPacket = Options.AudioChannelCount * sizeof(int16); // 2 channels of signed int16 PCM
	AudioSettings.mFramesPerPacket = 1;
	AudioSettings.mBytesPerFrame = Options.AudioChannelCount * sizeof(int16);
	AudioSettings.mChannelsPerFrame = Options.AudioChannelCount;
	AudioSettings.mBitsPerChannel = sizeof(int16)*8;
	AudioSettings.mReserved = 0; // Padding

	CurrentAudioTime.value = Options.FrameNumberOffset * Options.FrameRate.Denominator;
	CurrentAudioTime.timescale = Options.AudioSampleRate;
	CurrentAudioTime.flags = kPRTimeFlags_Valid;
	CurrentAudioTime.epoch = 0;


	PRStatus Status = ProResAudioFormatDescriptionCreate(
		&AudioSettings,
		0,
		NULL,
		&AudioFormatDescription);

	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to create a AudioFormatDescription from settings for file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

	return true;
}

void FAppleProResEncoder::Finalize()
{
	if (bFinalized || !bInitialized)
	{
		return;
	}

	if (FileWriter)
	{
		PRStatus Status = ProResFileWriterMarkEndOfDataForTrack(FileWriter, VideoTrackId);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to Mark End of Data for Video Track for file writer at path: \"%ls\".", *Options.OutputFilename);
		}

		Status = ProResFileWriterMarkEndOfDataForTrack(FileWriter, TimecodeTrackId);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to Mark End of Data for Timecode Track for file writer at path: \"%ls\".", *Options.OutputFilename);
		}

		if (Options.bIncludeAudio)
		{
			Status = ProResFileWriterMarkEndOfDataForTrack(FileWriter, AudioTrackId);
			if (Status != 0)
			{
				UE_LOGF(LogAppleProResMedia, Error, "Failed to Mark End of Data for Audio Track for file writer at path: \"%ls\".", *Options.OutputFilename);
			}
		}

		// Add one more sample onto the end 
		CurrentVideoTime.value += Options.FrameRate.Denominator;
		Status = ProResFileWriterEndSession(FileWriter, CurrentVideoTime);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to End Writer Session for file writer at path: \"%ls\".", *Options.OutputFilename);
		}

		Status = ProResFileWriterFinish(FileWriter);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to Finish Writer Session for file writer at path: \"%ls\".", *Options.OutputFilename);
		}

		Status = ProResFileWriterInvalidate(FileWriter);
		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to Invalidate Writer for file writer at path: \"%ls\".", *Options.OutputFilename);
		}
	}

	// Release resources allocated during initialization
	ReleaseAndClearResources();

	bFinalized = true;
}

void FAppleProResEncoder::ReleaseAndClearResources()
{
	PRReleaseAndClear(&VideoFormatDescription);
	PRReleaseAndClear(&AudioFormatDescription);
	PRReleaseAndClear(&TimecodeFormatDescription);
	PRReleaseAndClear(&FileWriter);

	if (Encoder)
	{
		PRCloseEncoder(Encoder);
		Encoder = nullptr;
	}

	bInitialized = false;
}

bool FAppleProResEncoder::WriteFrame(const FImagePixelData* InPixelData)
{
	check(InPixelData);

	if (!ensureMsgf(bInitialized && !bFinalized, TEXT("WriteFrame should not be called if not initialized or after finalize! Initialized: %d Finalized: %d"), bInitialized, bFinalized))
	{
		return false;
	}

	// Ensure the incoming data matches the settings the encoder was initialized with.
	const bool bResolutionMatches = (Options.Width == InPixelData->GetSize().X) && (Options.Height == InPixelData->GetSize().Y);
	if (!ensureMsgf(bResolutionMatches, TEXT("Frames cannot be provided in a different resolution than the encoder was initialized with.")))
	{
		return false;
	}

	FDateTime StartTime = FDateTime::Now();

	TArray<uint8> IntermediateFrameBuffer;
	IntermediateFrameBuffer.AddZeroed(Options.Width * Options.Height * 8);
	
	if (!ensureMsgf(AppleProRes::ConvertPixelDataToRGBA4444(InPixelData, Options.bWriteAlpha, IntermediateFrameBuffer), TEXT("Unsupported FImagePixelData format. Not writing frame!")))
	{
		return false;
	}

	// Encode to a 4 byte per pixel buffer
	TArray<uint8> OutputFrameBuffer;
	OutputFrameBuffer.AddUninitialized(Options.Width * Options.Height * 4);

	// Calculate the time of our sample
	PRSampleTimingInfo SampleTimingInfo;
	SampleTimingInfo.duration.value = Options.FrameRate.Denominator;
	SampleTimingInfo.duration.timescale = Options.FrameRate.Numerator;
	SampleTimingInfo.duration.flags = kPRTimeFlags_Valid;
	SampleTimingInfo.duration.epoch = 0;
	SampleTimingInfo.timeStamp = CurrentVideoTime;

	CurrentVideoTime.value += Options.FrameRate.Denominator;

	// Build the Encoder Params
	PREncodingParams EncodingParams;
	EncodingParams.interlaceMode = AppleProRes::GetInterlaceMode(Options.ScanMode);
	EncodingParams.preserveAlpha = Options.bWriteAlpha;
	EncodingParams.proResType = AppleProRes::GetCodecType(Options.Codec);

	// Build the incoming data (now converted to destination format)
	PRSourceFrame SourceFrame;
	SourceFrame.baseAddr = (void*)IntermediateFrameBuffer.GetData();

	SourceFrame.format = kPRFormat_b64a; // 4:4:4:4 Full-range (0-65535) ARGB  16-bit (Big Endian);
	SourceFrame.width = Options.Width;
	SourceFrame.height = Options.Height;
	SourceFrame.rowBytes = Options.Width * 8; // Assumes 8 bit target buffer

	// Encode the frame. Encoder will return if all frames are actually opaque as well as the actual final size.
	int ActualCompressedFrameSize = 0;
	bool bAllOpaqueAlpha = false;

	// This takes and returns ints instead of PRStatus
	int ReturnValue = PREncodeFrame(Encoder, &EncodingParams, &SourceFrame, OutputFrameBuffer.GetData(), MaxCompressedFrameSize, &ActualCompressedFrameSize, &bAllOpaqueAlpha);
	if (ReturnValue != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to encode frame for Video Track for file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

	size_t SampleSize = ActualCompressedFrameSize;

#if PLATFORM_MAC
	// This API is more efficient on Mac
	PRStatus Status = ProResFileWriterAddEncoderSampleBufferToTrack(
		FileWriter,
		VideoTrackId,
		Encoder,
		SampleTimingInfo.timeStamp);
#else
	// Add the sample to the Video Track
	PRStatus Status = ProResFileWriterAddSampleBufferToTrack(
		FileWriter, 
		VideoTrackId,
		OutputFrameBuffer.GetData(),
		ActualCompressedFrameSize,
		/*Deallocator*/ nullptr,
		VideoFormatDescription,
		/*NumSamples*/ 1,
		/*NumTimingInfos*/ 1,
		&SampleTimingInfo,
		/*NumSampleSizes*/ 1,
		&SampleSize);
#endif

	if (Status != 0)
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to add sample to Video Track for file writer at path: \"%ls\".", *Options.OutputFilename);
		return false;
	}

	const FAppleProResEncoder::FTimecodePayload* TimecodePayload = InPixelData->GetPayload<FAppleProResEncoder::FTimecodePayload>();
	if (TimecodePayload)
	{
		uint32 Timecode;
		Timecode = _byteswap_ulong(TimecodePayload->ReferenceFrameNumber);

		SampleSize = sizeof(Timecode);
		Status = ProResFileWriterAddSampleBufferToTrack(
			FileWriter,
			TimecodeTrackId,
			&Timecode,
			sizeof(Timecode),
			nullptr,
			TimecodeFormatDescription,
			1,
			1,
			&SampleTimingInfo,
			1,
			&SampleSize);

		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to add sample to Timecode Track for file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}
	}

	FDateTime EndTime = FDateTime::Now();
	return true;
}

bool FAppleProResEncoder::WriteAudioSample(const TArrayView<int16>& InAudioSamples)
{
	// If we aren't including audio, then we just say it was a success so that callers don't get confused.
	if (!Options.bIncludeAudio)
	{
		return true;
	}

	if (!ensureMsgf(bInitialized && !bFinalized, TEXT("WriteAudioSample should not be called if not initialized or after finalize! Initialized: %d Finalized: %d"), bInitialized, bFinalized))
	{
		return false;
	}

	if (!ensureMsgf(Options.AudioChannelCount != 0, TEXT("AudioChannelCount can't be zero.")))
	{
		return false;
	}

	if (!ensureMsgf(AudioSamplesPerFrame != 0, TEXT("AudioSamplesPerFrame can't be zero.")))
	{
		return false;
	}

	int32 SampleCountPerChannel = InAudioSamples.Num() / Options.AudioChannelCount;
	int32 NumFramesOfData = SampleCountPerChannel / AudioSamplesPerFrame;

	int16* SourceDataPtr = (int16*)InAudioSamples.GetData();
	for (int32 SampleIdx = 0; SampleIdx < NumFramesOfData; SampleIdx++)
	{
		// The number of bytes per frame is channel count (due to interleaved audio) * sizeof(int16) (signed 16-bit audio) * AudioSamplesPerFrame.
		const size_t BufferSize = AudioSamplesPerFrame * Options.AudioChannelCount * sizeof(int16);
		int32 NumSamplesInBuffer = AudioSamplesPerFrame;
		PRStatus Status = ProResFileWriterAddAudioSampleBufferToTrack(
			FileWriter,
			AudioTrackId,
			(void*)SourceDataPtr,
			BufferSize,
			/*Deallocator*/ nullptr,
			AudioFormatDescription,
			NumSamplesInBuffer,
			CurrentAudioTime
			);

		// Increment our source pointer to the next frame's data.
		SourceDataPtr += AudioSamplesPerFrame * Options.AudioChannelCount;

		// Increment our CurrentAudioTime (which is in hz, ie: 48'000) by 1 frame's worth of samples.
		CurrentAudioTime.value += AudioSamplesPerFrame;

		if (Status != 0)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to add sample to Audio Track for file writer at path: \"%ls\".", *Options.OutputFilename);
			return false;
		}

	}

	return true;
}

const FAppleProResEncoderOptions& FAppleProResEncoder::GetOptions() const
{
	return Options;
}
