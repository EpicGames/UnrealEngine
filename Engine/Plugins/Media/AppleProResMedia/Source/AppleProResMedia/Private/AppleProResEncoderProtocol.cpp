// Copyright Epic Games, Inc. All Rights Reserved.

#include "AppleProResEncoderProtocol.h"

#include "AppleProResMediaModule.h"
#include "Engine/RendererSettings.h"
#include "HAL/FileManager.h"
#include "MediaShaders.h"
#include "Misc/ScopeExit.h"
#include "MovieSceneCaptureProtocolBase.h"

UAppleProResEncoderProtocol::UAppleProResEncoderProtocol(const FObjectInitializer& InObjInit)
	: Super(InObjInit)
	, EncodingFormat(EAppleProResEncoderFormats::F_4444)
	, ColorDescription(EAppleProResEncoderColorDescription::CD_HDREC709)
	, ScanType(EAppleProResEncoderScanType::IM_PROGRESSIVE_SCAN)
	, NumberOfEncodingThreads(1)
	, bEmbedTimecodeTrack(false)
	, FileWriter(nullptr)
	, VideoTrackID(1)
	, TimecodeTrackID(2)
	, FormatDescription(nullptr)
	, TimecodeFormatDescription(nullptr)
	, Encoder(nullptr)
	, MaxCompressedFrameSize(1)
	, TargetCompressedFrameSize(1)
{
}

bool UAppleProResEncoderProtocol::SetupImpl()
{
	ParseCommandLine();

	return Super::SetupImpl();
}

struct FVideoFrameData : IFramePayload
{
	FFrameMetrics Metrics;
};

FFramePayloadPtr UAppleProResEncoderProtocol::GetFramePayload(const FFrameMetrics& InFrameMetrics)
{
	TSharedRef<FVideoFrameData, ESPMode::ThreadSafe> FrameData = MakeShareable(new FVideoFrameData);
	FrameData->Metrics = InFrameMetrics;
	return FrameData;
}

void UAppleProResEncoderProtocol::ParseCommandLine()
{
	int32 DecodedValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("-EncodingFormat="), DecodedValue))
	{
		DecodedValue = StaticEnum<EAppleProResEncoderFormats>()->GetIndexByValue(DecodedValue);
		if (DecodedValue != INDEX_NONE)
		{
			EncodingFormat = static_cast<EAppleProResEncoderFormats>(DecodedValue);
		}
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-ColorDescription="), DecodedValue))
	{
		DecodedValue = StaticEnum<EAppleProResEncoderColorDescription>()->GetIndexByValue(DecodedValue);
		if (DecodedValue != INDEX_NONE)
		{
			ColorDescription = static_cast<EAppleProResEncoderColorDescription>(DecodedValue);
		}
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-ScanType="), DecodedValue))
	{
		DecodedValue = StaticEnum<EAppleProResEncoderScanType>()->GetIndexByValue(DecodedValue);
		if (DecodedValue != INDEX_NONE)
		{
			ScanType = static_cast<EAppleProResEncoderScanType>(DecodedValue);
		}
	}

	FString BoolValue;
	if (FParse::Value(FCommandLine::Get(), TEXT("-EmbedTimecodeTrack="), BoolValue))
	{
		bEmbedTimecodeTrack = BoolValue.ToBool();
	}
	if (FParse::Value(FCommandLine::Get(), TEXT("-DropFrameTimecode="), BoolValue))
	{
		bDropFrameTimecode = BoolValue.ToBool();
	}
}

bool UAppleProResEncoderProtocol::CreateProResFile(const FString& InFilename)
{
	check(FileWriter == nullptr);

	if (CaptureHost == nullptr)
	{
		return false;
	}

	// Manually loading the toolbox library, on demand only.
	if (!UE::AppleProResMedia::LoadProResToolbox())
	{
		UE_LOGF(LogAppleProResMedia, Error, "Failed to initialize ProRes Encoder.");
		return false;
	}

	const auto Filename = StringCast<ANSICHAR>(*FPaths::ConvertRelativePathToFull(InFilename));

	Encoder = PROpenEncoder(NumberOfEncodingThreads, nullptr);

	PRStatus status = 0;
	status = ProResFileWriterCreate(Filename.Get(), &FileWriter);

	ON_SCOPE_EXIT
	{
		// Clear any allocated resources if initialization failed.
		if (status != 0)
		{
			ReleaseAndClearResources();
		}
	};

	if (status != 0)
	{
		return false;
	}


	FrameRate = CaptureHost->GetCaptureFrameRate();

	PRTimeScale TimeScale = FrameRate.Numerator;
	ProResFileWriterSetMovieTimescale(FileWriter, TimeScale);

	ProResFileWriterAddTrack(FileWriter, kPRMediaType_Video, &VideoTrackID);
	ProResFileWriterSetTrackMediaTimescale(FileWriter, VideoTrackID, TimeScale);

	if (bEmbedTimecodeTrack)
	{
		ProResFileWriterAddTrack(FileWriter, kPRMediaType_Timecode, &TimecodeTrackID);
		ProResFileWriterSetTrackMediaTimescale(FileWriter, TimecodeTrackID, TimeScale);
	}

	CurrentTime.value = 0;
	CurrentTime.timescale = FrameRate.Numerator;
	CurrentTime.flags = kPRTimeFlags_Valid;
	CurrentTime.epoch = 0;

	if (CaptureHost->GetSettings().bUseRelativeFrameNumbers == false)
	{
		CurrentTime.value += CaptureHost->GetFrameNumberOffset() * FrameRate.Denominator;
	}

	ProResFileWriterBeginSession(FileWriter, CurrentTime);

	Dimensions.width = CaptureHost->GetSettings().Resolution.ResX;
	Dimensions.height = CaptureHost->GetSettings().Resolution.ResY;

	{
		// We receive a 8bit buffer from FrameGrabber. We give a 16bit buffer to ProRes.
		//See ConvertFColorToRGBA4444
		int32 BufferSize = Dimensions.width * Dimensions.height;
		IntermediateFrameBuffer.AddZeroed(BufferSize * 8);
		OutputFrameBuffer.AddUninitialized(BufferSize * 4);
	}

	PRGetCompressedFrameSize(GetSelectedCodecType(), true, Dimensions.width, Dimensions.height, &MaxCompressedFrameSize, &TargetCompressedFrameSize);

	const int32 MaxChunkSize = 4 * 1024 * 1024;
	const int32 MaxFrameCount = FMath::Max(MaxChunkSize / TargetCompressedFrameSize, 1);

	ProResFileWriterSetTrackPreferredChunkSize(FileWriter, VideoTrackID, MaxFrameCount * TargetCompressedFrameSize);

	const ProResFormatDescriptionColorPrimaries ColorPrimaries = GetColorPrimaries();
	const ProResFormatDescriptionTransferFunction TransferFunction = GetTransferFunction();
	const ProResFormatDescriptionYCbCrMatrix Matrix = GetYCbCrMatrix();

	status = ProResVideoFormatDescriptionCreate(
		GetSelectedVideoCodecType(),
		Dimensions,
		GetBitDepth(),
		GetFieldCount(),
		GetFieldDetail(),
		ColorPrimaries,
		TransferFunction,
		Matrix,
		GetPixelAspectRatioHorizontalSpacing(),
		GetPixelAspectRatioVerticalSpacing(),
		nullptr,
		false,
		0.0f,
		&FormatDescription);

	if (status != 0)
	{
		return false;
	}

	{
		int32_t Primaries = GetFrameHeaderColorPrimaries();
		int32_t Characteristic = GetFrameHeaderTransferCharacteristic();
		int32_t MatrixCoefficients = GetFrameHeaderMatrixCoefficients();
		PRSetEncoderProperty(Encoder, kPRPropertyID_FrameHeaderColorPrimaries, sizeof(int32_t), &Primaries);
		PRSetEncoderProperty(Encoder, kPRPropertyID_FrameHeaderTransferCharacteristic, sizeof(int32_t), &Characteristic);
		PRSetEncoderProperty(Encoder, kPRPropertyID_FrameHeaderMatrixCoefficients, sizeof(int32_t), &MatrixCoefficients);
	}

	if (bEmbedTimecodeTrack)
	{
		PRTime FrameDuration;
		FrameDuration.value = FrameRate.Denominator;
		FrameDuration.timescale = FrameRate.Numerator;
		FrameDuration.flags = kPRTimeFlags_Valid;
		FrameDuration.epoch = 0;

		const FFrameRate TwentyNineNineSeven = FFrameRate(30000, 1001);

		uint32_t FrameQuanta = FMath::CeilToInt((float)FrameRate.Numerator / (float)FrameRate.Denominator);
		status = ProResTimecodeFormatDescriptionCreate(
			FrameDuration,
			FrameQuanta,
			(bDropFrameTimecode && (FrameRate == TwentyNineNineSeven)) ? kProResTimecodeFlag_DropFrame : 0,
			nullptr,
			0,
			0,
			&TimecodeFormatDescription);

		if (status != 0)
		{
			return false;
		}
	}

	return true;
}

void UAppleProResEncoderProtocol::ProcessFrame(FCapturedFrameData InFrame)
{
	// Defer writer creation until sequence has evaluated so that cached data for format mappings can be generated
	if (FileWriter == nullptr)
	{
		if (!CreateProResFile(GenerateFilenameImpl(FFrameMetrics(), TEXT(".mov"))))
		{
			return;
		}
	}

	FIntPoint ImageSize = FIntPoint(Dimensions.width, Dimensions.height);

	FVideoFrameData* Payload = InFrame.GetPayload<FVideoFrameData>();
	int32 FrameNumber = Payload->Metrics.FrameNumber;

	if (CaptureHost->GetSettings().bUseRelativeFrameNumbers == false)
	{
		FrameNumber += CaptureHost->GetFrameNumberOffset();
	}

	CurrentTime.value = FrameNumber * FrameRate.Denominator;

	PRSampleTimingInfo SampleTimingInfo;
	SampleTimingInfo.duration.value = FrameRate.Denominator;
	SampleTimingInfo.duration.timescale = FrameRate.Numerator;
	SampleTimingInfo.duration.flags = kPRTimeFlags_Valid;
	SampleTimingInfo.duration.epoch = 0;
	SampleTimingInfo.timeStamp = CurrentTime;

	PREncodingParams EncodingParams;
	PRSourceFrame SourceFrame;

	EncodingParams.interlaceMode = GetInterlaceMode();
	EncodingParams.preserveAlpha = true;
	EncodingParams.proResType = GetSelectedCodecType();

	FDateTime StartTime = FDateTime::Now();


	int32 Stride = GetStride();

	ConvertFColorToRGBA4444(InFrame.ColorBuffer);

	FDateTime ConvertionTime = FDateTime::Now();

	SourceFrame.baseAddr = (void*)IntermediateFrameBuffer.GetData();

	SourceFrame.format = GetPixelFormat();
	SourceFrame.width = ImageSize.X;
	SourceFrame.height = ImageSize.Y;
	SourceFrame.rowBytes = Stride;

	int ActualCompressedFrameSize = 0;
	bool bAllOpaqueAlpha = false;

	int ReturnValue = PREncodeFrame(Encoder, &EncodingParams, &SourceFrame, OutputFrameBuffer.GetData(), MaxCompressedFrameSize, &ActualCompressedFrameSize, &bAllOpaqueAlpha);

	if (ReturnValue == 0)
	{
		size_t sampleSize = ActualCompressedFrameSize;

		PRStatus Status = ProResFileWriterAddSampleBufferToTrack(
			FileWriter,
			VideoTrackID,
			OutputFrameBuffer.GetData(),
			ActualCompressedFrameSize,
			nullptr,
			FormatDescription,
			1,
			1,
			&SampleTimingInfo,
			1,
			&sampleSize);

		if (bEmbedTimecodeTrack)
		{
			uint32 Timecode;
			Timecode = _byteswap_ulong(FrameNumber);

			sampleSize = sizeof(Timecode);
			Status = ProResFileWriterAddSampleBufferToTrack(
				FileWriter,
				TimecodeTrackID,
				&Timecode,
				sizeof(Timecode),
				nullptr,
				TimecodeFormatDescription,
				1,
				1,
				&SampleTimingInfo,
				1,
				&sampleSize);
		}

		FDateTime EndTime = FDateTime::Now();
		FTimespan ConversionDeltaTime = ConvertionTime - StartTime;
		FTimespan CodecDeltaTime = EndTime - ConvertionTime;
		FTimespan TotalDeltaTime = EndTime - StartTime;

		UE_LOGF(LogAppleProResMedia, Verbose, "Processing Frame:%dx%d Frame:%d Conversion:%fms Codec:%fms Total:%fms",
			InFrame.BufferSize.X, 
			InFrame.BufferSize.Y, 
			FrameNumber,
			ConversionDeltaTime.GetTotalMilliseconds(),
			CodecDeltaTime.GetTotalMilliseconds(),
			TotalDeltaTime.GetTotalMilliseconds());
	}
	else
	{
		UE_LOGF(LogAppleProResMedia, Error, "Unable to encode Frame:%lld", CurrentTime.value);
	}
}

void UAppleProResEncoderProtocol::FinalizeImpl()
{
	Super::FinalizeImpl();
	if (FileWriter)
	{
		PRStatus status = ProResFileWriterMarkEndOfDataForTrack(FileWriter, VideoTrackID);
		if (bEmbedTimecodeTrack)
		{
			status = ProResFileWriterMarkEndOfDataForTrack(FileWriter, TimecodeTrackID);
		}
		CurrentTime.value += FrameRate.Denominator;
		status = ProResFileWriterEndSession(FileWriter, CurrentTime);
		status = ProResFileWriterFinish(FileWriter);
	}

	ReleaseAndClearResources();
}

bool UAppleProResEncoderProtocol::CanWriteToFileImpl(const TCHAR* InFilename, bool bInOverwriteExisting) const
{
	if (!bInOverwriteExisting)
	{
		return IFileManager::Get().FileSize(InFilename) == -1;
	}

	// Delete file since ProRes writer will be unable to overwrite it.
	IFileManager::Get().Delete(InFilename);

	return true;
}

PRCodecType UAppleProResEncoderProtocol::GetSelectedCodecType() const
{
	switch (EncodingFormat)
	{
	case EAppleProResEncoderFormats::F_422HQ:
		return kPRType422HQ;
	case EAppleProResEncoderFormats::F_422:
		return kPRType422;
	case EAppleProResEncoderFormats::F_422LT:
		return kPRType422LT;
	case EAppleProResEncoderFormats::F_422Proxy:
		return kPRType422Proxy;
	case EAppleProResEncoderFormats::F_4444:
		return kPRType4444;
	case EAppleProResEncoderFormats::F_4444XQ:
		return kPRType4444XQ;
	}
	ensureMsgf(false, TEXT("Invalid encoding format."));
	return kPRType422HQ;
}

PRVideoCodecType UAppleProResEncoderProtocol::GetSelectedVideoCodecType() const
{
	switch (EncodingFormat)
	{
	case EAppleProResEncoderFormats::F_422HQ:
		return kPRVideoCodecType_AppleProRes422HQ;
	case EAppleProResEncoderFormats::F_422:
		return kPRVideoCodecType_AppleProRes422;
	case EAppleProResEncoderFormats::F_422LT:
		return kPRVideoCodecType_AppleProRes422LT;
	case EAppleProResEncoderFormats::F_422Proxy:
		return kPRVideoCodecType_AppleProRes422Proxy;
	case EAppleProResEncoderFormats::F_4444:
		return kPRVideoCodecType_AppleProRes4444;
	case EAppleProResEncoderFormats::F_4444XQ:
		return kPRVideoCodecType_AppleProRes4444XQ;
	}
	ensureMsgf(false, TEXT("Invalid encoding format."));
	return kPRVideoCodecType_AppleProRes4444XQ;
}

ProResFormatDescriptionColorPrimaries UAppleProResEncoderProtocol::GetColorPrimaries() const
{
	switch (ColorDescription)
	{
	case EAppleProResEncoderColorDescription::CD_SDREC601_525_60HZ:
		return kProResFormatDescriptionColorPrimaries_SMPTE_C;
	case EAppleProResEncoderColorDescription::CD_SDREC601_625_50HZ:
		return kProResFormatDescriptionColorPrimaries_EBU_3213;
	case EAppleProResEncoderColorDescription::CD_HDREC709:
		return kProResFormatDescriptionColorPrimaries_ITU_R_709_2;
	}
	ensureMsgf(false, TEXT("Invalid color description."));
	return kProResFormatDescriptionColorPrimaries_SMPTE_C;
}

int32_t UAppleProResEncoderProtocol::GetFrameHeaderColorPrimaries() const
{
	switch (ColorDescription)
	{
	case EAppleProResEncoderColorDescription::CD_SDREC601_525_60HZ:
		return kPRColorPrimaries_SMPTE_C;
	case EAppleProResEncoderColorDescription::CD_SDREC601_625_50HZ:
		return kPRColorPrimaries_EBU_3213;
	case EAppleProResEncoderColorDescription::CD_HDREC709:
		return kPRColorPrimaries_ITU_R_709;
	}
	ensureMsgf(false, TEXT("Invalid color description."));
	return kPRColorPrimaries_Unspecified;
}

ProResFormatDescriptionTransferFunction UAppleProResEncoderProtocol::GetTransferFunction() const
{
	switch (ColorDescription)
	{
	case EAppleProResEncoderColorDescription::CD_SDREC601_525_60HZ:
	case EAppleProResEncoderColorDescription::CD_SDREC601_625_50HZ:
	case EAppleProResEncoderColorDescription::CD_HDREC709:
		return kProResFormatDescriptionTransferFunction_ITU_R_709_2;
	}
	ensureMsgf(false, TEXT("Invalid color description."));
	return kProResFormatDescriptionTransferFunction_ITU_R_709_2;
}

int32_t UAppleProResEncoderProtocol::GetFrameHeaderTransferCharacteristic() const
{
	switch (ColorDescription)
	{
	case EAppleProResEncoderColorDescription::CD_SDREC601_525_60HZ:
	case EAppleProResEncoderColorDescription::CD_SDREC601_625_50HZ:
	case EAppleProResEncoderColorDescription::CD_HDREC709:
		return kPRTransferCharacteristic_ITU_R_709;
	}
	ensureMsgf(false, TEXT("Invalid color description."));
	return kPRTransferCharacteristic_Unspecified;
}

ProResFormatDescriptionYCbCrMatrix UAppleProResEncoderProtocol::GetYCbCrMatrix() const
{
	switch (ColorDescription)
	{
	case EAppleProResEncoderColorDescription::CD_SDREC601_525_60HZ:
	case EAppleProResEncoderColorDescription::CD_SDREC601_625_50HZ:
		return kProResFormatDescriptionYCbCrMatrix_ITU_R_601_4;
	case EAppleProResEncoderColorDescription::CD_HDREC709:
		return kProResFormatDescriptionYCbCrMatrix_ITU_R_709_2;
	}
	ensureMsgf(false, TEXT("Invalid color description."));
	return kProResFormatDescriptionYCbCrMatrix_ITU_R_709_2;
}

int32_t UAppleProResEncoderProtocol::GetFrameHeaderMatrixCoefficients() const
{
	switch (ColorDescription)
	{
	case EAppleProResEncoderColorDescription::CD_SDREC601_525_60HZ:
	case EAppleProResEncoderColorDescription::CD_SDREC601_625_50HZ:
		return kPRMatrixCoefficients_ITU_R_601;
	case EAppleProResEncoderColorDescription::CD_HDREC709:
		return kPRMatrixCoefficients_ITU_R_709;
	}
	ensureMsgf(false, TEXT("Invalid color description."));
	return kPRMatrixCoefficients_Unspecified;
}

PRInterlaceMode UAppleProResEncoderProtocol::GetInterlaceMode() const
{
	switch (ScanType)
	{
	case EAppleProResEncoderScanType::IM_PROGRESSIVE_SCAN:
		return kPRProgressiveScan;
	case EAppleProResEncoderScanType::IM_INTERLACED_TOP_FIELD_FIRST:
		return kPRInterlacedTopFieldFirst;
	case EAppleProResEncoderScanType::IM_INTERLATED_BOTTOM_FIRST_FIRST:
		return kPRInterlacedBottomFieldFirst;
	}
	ensureMsgf(false, TEXT("Invalid scan type."));
	return kPRProgressiveScan;
}

PRPixelFormat UAppleProResEncoderProtocol::GetPixelFormat() const
{
	return kPRFormat_b64a;
}

uint32 UAppleProResEncoderProtocol::GetPixelAspectRatioHorizontalSpacing() const
{
	return 1;
}

uint32 UAppleProResEncoderProtocol::GetPixelAspectRatioVerticalSpacing() const
{
	return 1;
}

uint32_t UAppleProResEncoderProtocol::GetBitDepth() const
{
	switch (EncodingFormat)
	{
	case EAppleProResEncoderFormats::F_4444:
		return HasAlpha() ? 32 : 24;
	default:
		return 24;
	}
}

uint32_t UAppleProResEncoderProtocol::GetFieldCount() const
{
	switch (ScanType)
	{
	case EAppleProResEncoderScanType::IM_PROGRESSIVE_SCAN:
		return 1;
	case EAppleProResEncoderScanType::IM_INTERLACED_TOP_FIELD_FIRST:
	case EAppleProResEncoderScanType::IM_INTERLATED_BOTTOM_FIRST_FIRST:
		return 2;
	}
	ensureMsgf(false, TEXT("Invalid scan type."));
	return 1;
}

uint32 UAppleProResEncoderProtocol::GetStride() const
{
	return Dimensions.width * 8;
}

ProResFormatDescriptionFieldDetail UAppleProResEncoderProtocol::GetFieldDetail() const
{
	switch (ScanType)
	{
	case EAppleProResEncoderScanType::IM_PROGRESSIVE_SCAN:
		return kProResFormatDescriptionFieldDetail_Unknown;
	case EAppleProResEncoderScanType::IM_INTERLACED_TOP_FIELD_FIRST:
		return kProResFormatDescriptionFieldDetail_SpatialFirstLineEarly;
	case EAppleProResEncoderScanType::IM_INTERLATED_BOTTOM_FIRST_FIRST:
		return kProResFormatDescriptionFieldDetail_SpatialFirstLineLate;
	}
	ensureMsgf(false, TEXT("Invalid scan type."));
	return kProResFormatDescriptionFieldDetail_Unknown;
}

bool UAppleProResEncoderProtocol::HasAlpha() const
{
	static const auto CVarPropagateAlpha = IConsoleManager::Get().FindConsoleVariable(TEXT("r.PostProcessing.PropagateAlpha"));
	return CVarPropagateAlpha->GetBool();
}

void UAppleProResEncoderProtocol::ConvertFColorToRGBA4444(const TArray<FColor>& InColorbuffer)
{
	int32 Size = InColorbuffer.Num();
	uint8* SourceBuffer = (uint8*)InColorbuffer.GetData();
	uint8* DestinationBuffer = (uint8*)IntermediateFrameBuffer.GetData();

	const uint8 AlphaMultiplier = HasAlpha() ? 1 : 0;

	for (int32 i = 0; i < Size; i++)
	{
		DestinationBuffer[0] = 255 - SourceBuffer[3] * AlphaMultiplier;	// A
		DestinationBuffer[2] = SourceBuffer[2];	// R
		DestinationBuffer[4] = SourceBuffer[1]; // G
		DestinationBuffer[6] = SourceBuffer[0]; // B
		DestinationBuffer += 8;
		SourceBuffer += 4;
	}
}

void UAppleProResEncoderProtocol::ReleaseAndClearResources()
{
	PRReleaseAndClear(&FormatDescription);
	PRReleaseAndClear(&TimecodeFormatDescription);
	PRReleaseAndClear(&FileWriter);

	if (Encoder)
	{
		PRCloseEncoder(Encoder);
		Encoder = nullptr;
	}
}