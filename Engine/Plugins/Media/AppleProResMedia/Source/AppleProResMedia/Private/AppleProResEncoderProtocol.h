// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "Containers/Array.h"

#define __LLP64__ 1

#include "ProResProperties.h"
#include "ProResEncoder.h"
#include "ProResFileWriter.h"

#include "Protocols/FrameGrabberProtocol.h"

#include "UObject/StrongObjectPtr.h"

#include "AppleProResEncoderProtocol.generated.h"

#define UE_API APPLEPRORESMEDIA_API

UENUM()
enum class EAppleProResEncoderFormats : uint8
{
	F_422HQ UMETA(DisplayName = "422 HQ"),
	F_422 UMETA(DisplayName = "422"),
	F_422LT UMETA(DisplayName = "422 LT"),
	F_422Proxy UMETA(DisplayName = "422 Proxy"),
	F_4444 UMETA(DisplayName = "4444"),
	F_4444XQ UMETA(DisplayName = "4444 XQ"),
};

UENUM()
enum class EAppleProResEncoderColorDescription : uint8
{
	CD_SDREC601_525_60HZ UMETA(DisplayName = "SD Rec. 601 525/60Hz"),
	CD_SDREC601_625_50HZ UMETA(DisplayName = "SD Rec. 601 625/50Hz"),
	CD_HDREC709 UMETA(DisplayName = "HD Rec. 709"),
};

UENUM()
enum class EAppleProResEncoderScanType : uint8
{
	IM_PROGRESSIVE_SCAN UMETA(DisplayName = "Progressive encoding mode"),
	IM_INTERLACED_TOP_FIELD_FIRST UMETA(DisplayName = "Interlaced mode; first (top) image line belongs to first temporal field"),
	IM_INTERLATED_BOTTOM_FIRST_FIRST UMETA(DisplayName = "Interlaced mode; second (bottom) image line belongs to first temporal field"),
};

UCLASS(MinimalAPI, meta = (DisplayName = "Apple ProRes Encoder (mov)", CommandLineID = "AppleProRes") )
class UAppleProResEncoderProtocol : public UFrameGrabberProtocol
{
public:
	GENERATED_BODY()

	UE_API UAppleProResEncoderProtocol(const FObjectInitializer& InObjInit);

public:
	UE_API virtual bool SetupImpl() override;
	UE_API virtual void FinalizeImpl() override;
	UE_API virtual FFramePayloadPtr GetFramePayload(const FFrameMetrics& InFrameMetrics) override;
	UE_API virtual void ProcessFrame(FCapturedFrameData InFrame) override;
	UE_API virtual bool CanWriteToFileImpl(const TCHAR* InFilename, bool bInOverwriteExisting) const override;

public:

	UPROPERTY(config, EditAnywhere, Category = "Apple ProRes Settings")
	EAppleProResEncoderFormats EncodingFormat;

	UPROPERTY(config, EditAnywhere, Category = "Apple ProRes Settings")
	EAppleProResEncoderColorDescription ColorDescription;

	UPROPERTY(config, EditAnywhere, Category = "Apple ProRes Settings")
	EAppleProResEncoderScanType ScanType;

	UPROPERTY(config, EditAnywhere, Category = "Apple ProRes Settings", meta = (ClampMin = "0.0", UIMin = "0.0", ClampMax = "64.0", UIMax = "64.0"))
	int32 NumberOfEncodingThreads;

	UPROPERTY(config, EditAnywhere, Category = "Apple ProRes Settings")
	bool bEmbedTimecodeTrack;

	/** Use Drop Frame Timecode when applicable (29.97p or 59.94i). */
	UPROPERTY(config, EditAnywhere, Category = "Apple ProRes Settings")
	bool bDropFrameTimecode;

private:

	UE_API void ParseCommandLine();
	UE_API bool CreateProResFile(const FString& InFilename);
	UE_API void ConvertFColorToRGBA4444(const TArray<FColor>& InColorbuffer);

	UE_API PRCodecType GetSelectedCodecType() const;
	UE_API PRVideoCodecType GetSelectedVideoCodecType() const;
	UE_API ProResFormatDescriptionColorPrimaries GetColorPrimaries() const;
	UE_API ProResFormatDescriptionTransferFunction GetTransferFunction() const;
	UE_API ProResFormatDescriptionYCbCrMatrix GetYCbCrMatrix() const;
	UE_API int32_t GetFrameHeaderColorPrimaries() const;
	UE_API int32_t GetFrameHeaderTransferCharacteristic() const;
	UE_API int32_t GetFrameHeaderMatrixCoefficients() const;
	UE_API PRInterlaceMode GetInterlaceMode() const;
	UE_API PRPixelFormat GetPixelFormat() const;
	UE_API uint32 GetPixelAspectRatioHorizontalSpacing() const;
	UE_API uint32 GetPixelAspectRatioVerticalSpacing() const;
	UE_API uint32 GetStride() const;
	UE_API uint32_t GetBitDepth() const;
	UE_API uint32_t GetFieldCount() const;
	UE_API ProResFormatDescriptionFieldDetail GetFieldDetail() const;
	UE_API bool HasAlpha() const;
	UE_API void ReleaseAndClearResources();

private:

	ProResFileWriterRef FileWriter;
	PRPersistentTrackID VideoTrackID;
	PRPersistentTrackID TimecodeTrackID;
	PRTime CurrentTime;
	ProResFormatDescriptionRef FormatDescription;
	ProResTimecodeFormatDescriptionRef TimecodeFormatDescription;
	PREncoderRef Encoder;
	PRVideoDimensions Dimensions;

	FFrameRate FrameRate;
	int32 MaxCompressedFrameSize;
	int32 TargetCompressedFrameSize;
	TArray<uint8> IntermediateFrameBuffer;
	TArray<uint8> OutputFrameBuffer;
};

#undef UE_API
