// Copyright Epic Games, Inc. All Rights Reserved.


#include "AppleProResMediaModule.h"

#include "AppleProResEncoderProtocol.h"

#include "Interfaces/IPluginManager.h"

#include "Misc/Paths.h"

#include "Modules/ModuleManager.h"

#if PLATFORM_WINDOWS
#include "Windows/WmfMediaAppleProResDecoder.h"

#include "IWmfMediaModule.h"

#include "WmfMediaCodec/WmfMediaCodecGenerator.h"
#include "WmfMediaCodec/WmfMediaCodecManager.h"

#include "WmfMediaCommon.h"
#endif

#if WITH_EDITOR
	#include "AppleProResMediaSettings.h"
	#include "ISettingsModule.h"
#endif

#define LOCTEXT_NAMESPACE "ProRes"

DEFINE_LOG_CATEGORY(LogAppleProResMedia);

class FAppleProResMediaModule : public IModuleInterface
{
public:
	bool LoadProResToolbox()
	{
#if PLATFORM_WINDOWS
		if (ProResToolboxLibHandle)
		{
			return true;
		}
	
		const FString ProResDllName = TEXT("ProResToolbox.dll");

		// determine directory paths

		TSharedPtr<IPlugin> ThisPlugin = IPluginManager::Get().FindPlugin(TEXT("AppleProResMedia"));
		if (!ThisPlugin)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to find the plugin folder for \"%ls\". Plug-in will not be fully functional.", *ProResDllName);
			return false;
		}

		const FString ProResDllPath = FPaths::Combine(ThisPlugin->GetBaseDir(), TEXT("Binaries"), TEXT("ThirdParty"), TEXT("Win64"));
		const FString ProResDllFullPath = FPaths::Combine(ProResDllPath, ProResDllName);
		
		if (!FPaths::FileExists(ProResDllFullPath))
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to find required library \"%ls\". Plug-in will not be fully functional.", *ProResDllFullPath);
			return false;
		}

		FPlatformProcess::PushDllDirectory(*ProResDllPath);
		ProResToolboxLibHandle = FPlatformProcess::GetDllHandle(*ProResDllFullPath);
		FPlatformProcess::PopDllDirectory(*ProResDllPath);
		
		if (ProResToolboxLibHandle == nullptr)
		{
			UE_LOGF(LogAppleProResMedia, Error, "Failed to load required library \"%ls\". Plug-in will not be fully functional.", *ProResDllFullPath);
			return false;
		}
#endif
		return true;
	}
	
	virtual void StartupModule() override
	{
#if PLATFORM_WINDOWS
		if (IWmfMediaModule* Module = IWmfMediaModule::Get())
		{
			if (Module->IsInitialized())
			{
				Module->GetCodecManager()->AddCodec(MakeUnique<WmfMediaCodecGenerator<WmfMediaAppleProResDecoder>>(true));
			}
		}
#endif

#if WITH_EDITOR
		// register settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->RegisterSettings("Project", "Plugins", "AppleProResMedia",
				LOCTEXT("AppleProResMediaSettingsName", "Apple ProRes Media"),
				LOCTEXT("AppleProResMediaSettingsDescription", "Configure the Apple ProRes Media plug-in."),
				GetMutableDefault<UAppleProResMediaSettings>()
			);
		}
#endif //WITH_EDITOR

		// Add exemption to FName::NameToDisplayString formatting to ensure "ProRes" is displayed without a space
		FName::AddNameToDisplayStringExemption(TEXT("ProRes"));
	}

	virtual void ShutdownModule() override
	{
#if WITH_EDITOR
		// unregister settings
		ISettingsModule* SettingsModule = FModuleManager::GetModulePtr<ISettingsModule>("Settings");

		if (SettingsModule != nullptr)
		{
			SettingsModule->UnregisterSettings("Project", "Plugins", "AppleProResMedia");
		}
#endif //WITH_EDITOR

		if (ProResToolboxLibHandle != nullptr)
		{
			FPlatformProcess::FreeDllHandle(ProResToolboxLibHandle);
			ProResToolboxLibHandle = nullptr;
		}
	}

	// Codec could still be in use
	virtual bool SupportsDynamicReloading()
	{
		return false;
	}

private:

	static void* ProResToolboxLibHandle;
};

void* FAppleProResMediaModule::ProResToolboxLibHandle = nullptr;

bool UE::AppleProResMedia::LoadProResToolbox()
{
#if PLATFORM_WINDOWS
	if (FAppleProResMediaModule* AppleProResMediaModule = FModuleManager::GetModulePtr<FAppleProResMediaModule>("AppleProResMedia"))
	{
		return AppleProResMediaModule->LoadProResToolbox();
	}
	return false;
#else
	return true;	// There is nothing to do for other platforms
#endif
}

IMPLEMENT_MODULE(FAppleProResMediaModule, AppleProResMedia)

#undef LOCTEXT_NAMESPACE
