// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_MAC
#include <libkern/OSByteOrder.h>
#define _byteswap_ulong(x) OSSwapInt32(x)
#endif

DECLARE_LOG_CATEGORY_EXTERN(LogAppleProResMedia, Log, All);

namespace UE::AppleProResMedia
{
	/**
	 * ProResToolbox is delay loaded but needs to be manually loaded because of search path limitations.
	 * @return true if the library was loaded, false otherwise.
	 */
	bool LoadProResToolbox();
}