// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "AppleProResMediaSettings.generated.h"


/**
 * Settings for the AppleProResMedia plug-in.
 */
UCLASS(config=Engine, defaultconfig)
class UAppleProResMediaSettings
	: public UObject
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	UAppleProResMediaSettings();

public:

	/** Number of CPU decoding threads (Set this to 0 to have the decoder spawn according to the number of processors detected in the system.). */
	UPROPERTY(config, EditAnywhere, DisplayName = "Number Of CPU Decoding Threads", Category = Media, meta = (ClampMin = "0.0", UIMin = "0.0"))
	int32 NumberOfCPUDecodingThreads;

public:

};
