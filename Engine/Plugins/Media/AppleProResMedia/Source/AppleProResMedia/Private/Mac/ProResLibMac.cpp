// Copyright Epic Games, Inc. All Rights Reserved.
// Mac implementation of ProResToolbox and ProResLib, using AVFoundation and VideoToolbox

#include "ProResEncoder.h"
#include "ProResProperties.h"
#include "ProResFileReader.h"
#include "ProResFileWriter.h"
#include "ProResFormatDescription.h"
#include "ProResTime.h"
#include "ProResTypes.h"
#include "ProResToolboxExtension.h"

#if PLATFORM_MAC
#include "Mac/MacSystemIncludes.h"
#endif

#import <AVFoundation/AVFoundation.h>
#import <VideoToolbox/VideoToolbox.h>

struct PREncoder
{
	VTCompressionSessionRef session = NULL;
	CFMutableDictionaryRef sessionProperties = NULL;
	CMSampleBufferRef sampleBufferOut = NULL;
	
	PREncoder()
	{
		sessionProperties = CFDictionaryCreateMutable(kCFAllocatorDefault,
													  0,
													  &kCFTypeDictionaryKeyCallBacks,
													  &kCFTypeDictionaryValueCallBacks);
	}
	
	~PREncoder()
	{
		CFRelease(sessionProperties);
		sessionProperties = NULL;
	}
	
	static void pixelBufferReleaseCallback(void* releaseRefCon, const void* baseAddress)
	{
		delete [] (uint8*)baseAddress;
	}
};

struct OpaqueProResFileWriter
{
	AVAssetWriter* innerRef;
	
	OpaqueProResFileWriter(const char *destUTF8Path)
	{
		auto pool = [[NSAutoreleasePool alloc] init];
		innerRef = [[AVAssetWriter alloc] initWithURL:[NSURL fileURLWithPath:[NSString stringWithUTF8String:destUTF8Path] 
																 isDirectory:NO]
											 fileType:AVFileTypeQuickTimeMovie
												error:nil];
		[pool release];
	}
	
	~OpaqueProResFileWriter()
	{
		[innerRef release];
		innerRef = nil;
	}
};

struct opaqueFormatDescription
{
	CMFormatDescriptionRef innerRef = NULL;
	
	opaqueFormatDescription(CMFormatDescriptionRef ref) : innerRef(ref)
	{
	}
	
	~opaqueFormatDescription()
	{
		if (innerRef)
		{
			CFRelease(innerRef);
			innerRef = NULL;
		}
	}
};

AVMediaType GetAVMediaTypeFromPRMediaType(PRMediaType mediaType)
{
	switch (mediaType) {
		case kPRMediaType_Video:
			return AVMediaTypeVideo;
		case kPRMediaType_Audio:
			return AVMediaTypeAudio;
		case kPRMediaType_Timecode:
			return AVMediaTypeTimecode;
		default:
			return @"InvalidMediaType";
	}
}

CMTime CMTimeFromPRTime(PRTime time)
{
	return CMTimeMakeWithEpoch(time.value, time.timescale, time.epoch);
}

CMSampleTimingInfo CMSampleTimingInfoFromPRSampleTimingInfo(PRSampleTimingInfo info)
{
	return CMSampleTimingInfo{
		CMTimeFromPRTime(info.duration),
		CMTimeFromPRTime(info.timeStamp),
		kCMTimeInvalid
	};
}

#pragma mark - ProResEncoder

void
PRGetCompressedFrameSize(
		PRCodecType proResType,
		bool preserveAlpha,
		int frameWidth,
		int frameHeight,
		int* maxCompressedFrameSize,
		int* targetCompressedFrameSize)
{
	auto encoderSpecification = CFDictionaryCreateMutable(kCFAllocatorDefault,
														  0,
														  &kCFTypeDictionaryKeyCallBacks,
														  &kCFTypeDictionaryValueCallBacks);
	CFDictionaryAddValue(encoderSpecification,
						 kVTVideoEncoderSpecification_EnableHardwareAcceleratedVideoEncoder,
						 kCFBooleanTrue);
	VTCompressionSessionRef session;
	auto status = VTCompressionSessionCreate(kCFAllocatorDefault, 		/* allocator                   */
											 frameWidth,				/* width                       */
											 frameHeight,				/* height                      */
											 proResType,				/* codecType                   */	// gonna trust PRCodecType and CMVideoCodecType are using the same 4 char code
											 encoderSpecification,		/* encoderSpecification        */
											 NULL,						/* sourceImageBufferAttributes */
											 NULL,						/* compressedDataAllocator     */
											 NULL,						/* outputCallback              */
											 NULL,						/* outputCallbackRefCon        */
											 &session);					/* compressionSessionOut       */
	VTSessionSetProperty(session, kVTCompressionPropertyKey_PreserveAlphaChannel, preserveAlpha ? kCFBooleanTrue : kCFBooleanFalse);
	
	// no API to get maxCompressedFrameSize, assume uncompressed size
	*maxCompressedFrameSize = frameWidth * frameHeight * 4;
	*targetCompressedFrameSize = frameWidth * frameHeight * 4;
	if (status == noErr && session)
	{
		// try to read the estimate
		CFNumberRef bytesPerFrame;
		status = VTSessionCopyProperty(session, kVTCompressionPropertyKey_EstimatedAverageBytesPerFrame, kCFAllocatorDefault, &bytesPerFrame);
		if (status == noErr)
		{
			CFNumberGetValue(bytesPerFrame, kCFNumberIntType, targetCompressedFrameSize);
			CFRelease(bytesPerFrame);
		}
		else
		{
			*targetCompressedFrameSize = frameWidth * frameHeight * 4;
		}
		
		VTCompressionSessionInvalidate(session);
		CFRelease(session);
	}
	CFRelease(encoderSpecification);
}

PREncoderRef
PROpenEncoder(int numThreads, void (*threadStartupCallback)())
{
	// these parameters are not used, VTCompressionSession will use threading as needed
	return new PREncoder();
}

int PREncodeFrame(PREncoderRef encoder,
				  const PREncodingParams* encodingParams,
				  const PRSourceFrame* sourceFrame,
				  void* destinationPtr,
				  int   destinationSize,
				  int*  compressedFrameSize,
				  bool* allOpaqueAlpha)
{
	int status = 0;
	if (!encoder->session)
	{
		auto encoderSpecification = CFDictionaryCreateMutable(kCFAllocatorDefault,
															  0,
															  &kCFTypeDictionaryKeyCallBacks,
															  &kCFTypeDictionaryValueCallBacks);
		CFDictionaryAddValue(encoderSpecification,
							 kVTVideoEncoderSpecification_EnableHardwareAcceleratedVideoEncoder,
							 kCFBooleanTrue);
		
		status = VTCompressionSessionCreate(kCFAllocatorDefault, 				/* allocator                   */
											sourceFrame->width,					/* width                       */
											sourceFrame->height,				/* height                      */
											encodingParams->proResType,			/* codecType                   */	// gonna trust PRCodecType and CMVideoCodecType are using the same 4 char code
											encoderSpecification,				/* encoderSpecification        */
											NULL,								/* sourceImageBufferAttributes */
											NULL,								/* compressedDataAllocator     */
											NULL,								/* outputCallback              */
											NULL,								/* outputCallbackRefCon        */
											&encoder->session);					/* compressionSessionOut       */
		
		// configure session with properties
		if (encodingParams->interlaceMode == kPRProgressiveScan)
		{
			CFDictionaryAddValue(encoder->sessionProperties,
								 kVTCompressionPropertyKey_ProgressiveScan,
								 kCFBooleanTrue);
		}
		else
		{
			CFDictionaryAddValue(encoder->sessionProperties,
								 kVTCompressionPropertyKey_ProgressiveScan,
								 kCFBooleanFalse);
			int value = 2;
			CFDictionaryAddValue(encoder->sessionProperties,
								 kVTCompressionPropertyKey_FieldCount,
								 CFNumberCreate(kCFAllocatorDefault, kCFNumberIntType, &value));
			if (encodingParams->interlaceMode == kPRInterlacedTopFieldFirst)
			{
				CFDictionaryAddValue(encoder->sessionProperties,
									 kVTCompressionPropertyKey_FieldDetail,
									 kCMFormatDescriptionFieldDetail_TemporalTopFirst);
			}
			else if (encodingParams->interlaceMode == kPRInterlacedTopFieldFirst)
			{
				CFDictionaryAddValue(encoder->sessionProperties,
									 kVTCompressionPropertyKey_FieldDetail,
									 kCMFormatDescriptionFieldDetail_TemporalBottomFirst);
			}
		}
		CFDictionaryAddValue(encoder->sessionProperties,
							 kVTCompressionPropertyKey_PreserveAlphaChannel,
							 encodingParams->preserveAlpha ? kCFBooleanTrue : kCFBooleanFalse);
		// default color primaries
		if (!CFDictionaryContainsKey(encoder->sessionProperties, kVTCompressionPropertyKey_ColorPrimaries))
		{
			CFDictionaryAddValue(encoder->sessionProperties,
								 kVTCompressionPropertyKey_ColorPrimaries,
								 kCMFormatDescriptionColorPrimaries_ITU_R_709_2);
		}
		if (!CFDictionaryContainsKey(encoder->sessionProperties, kVTCompressionPropertyKey_TransferFunction))
		{
			CFDictionaryAddValue(encoder->sessionProperties,
								 kVTCompressionPropertyKey_TransferFunction,
								 kCMFormatDescriptionTransferFunction_ITU_R_709_2);
		}
		if (!CFDictionaryContainsKey(encoder->sessionProperties, kVTCompressionPropertyKey_YCbCrMatrix))
		{
			CFDictionaryAddValue(encoder->sessionProperties,
								 kVTCompressionPropertyKey_YCbCrMatrix,
								 kCMFormatDescriptionYCbCrMatrix_ITU_R_709_2);
		}
		VTSessionSetProperties(encoder->session, encoder->sessionProperties);
	}
	
	if (status == noErr && encoder->session) 
	{
		CVPixelBufferRef pixelBuffer;
		// copy the sourceFrame buffer, we need it to live beyond this call
		void* copiedFrameData = new uint8[sourceFrame->rowBytes * sourceFrame->height];
		memcpy(copiedFrameData, sourceFrame->baseAddr, sourceFrame->rowBytes * sourceFrame->height);
		status = CVPixelBufferCreateWithBytes(kCFAllocatorDefault,
											  sourceFrame->width,
											  sourceFrame->height,
											  sourceFrame->format,	// gonna trust PRPixelFormat and CVPixelFormatType are using the same 4 char code
											  copiedFrameData,
											  sourceFrame->rowBytes,
											  PREncoder::pixelBufferReleaseCallback,
											  copiedFrameData,
											  NULL,
											  &pixelBuffer);
		
		if (status == kCVReturnSuccess)
		{
			if (encoder->sampleBufferOut)
			{
				NSLog(@"Will overwrite encoder->sampleBufferOut! You should call ProResFileWriterAddEncoderSampleBufferToTrack before encoding the next frame!");
				CFRelease(encoder->sampleBufferOut);
			}
			encoder->sampleBufferOut = NULL;
			VTEncodeInfoFlags flags;
			status = VTCompressionSessionEncodeFrameWithOutputHandler(encoder->session,
																	  pixelBuffer,
																	  kCMTimeZero,	// the actual timing will be passed in during AddSampleBufferToTrack
																	  kCMTimeInvalid,
																	  NULL,
																	  &flags,
																	  ^(OSStatus encodeStatus, VTEncodeInfoFlags infoFlags, CMSampleBufferRef sampleBuffer) {
				if (encodeStatus == noErr && sampleBuffer)
				{
					// store sampleBuffer in sampleBufferOut to be written later
					CFRetain(sampleBuffer);
					encoder->sampleBufferOut = sampleBuffer;
				}
			});
			if (status == noErr && !(flags & kVTEncodeInfo_FrameDropped))
			{
				if (flags & kVTEncodeInfo_Asynchronous)
				{
					// wait for async encoding
					float Timer = 0;
					do {
						if ((encoder->sampleBufferOut && CMSampleBufferDataIsReady(encoder->sampleBufferOut)) || Timer > 1.0f)	// usually takes <0.01s
							break; 
						Timer += 0.001f;
						usleep(1000);	// 0.001 second
					} while (true);
				}
				
				if (!encoder->sampleBufferOut || !CMSampleBufferDataIsReady(encoder->sampleBufferOut) || CMSampleBufferGetTotalSampleSize(encoder->sampleBufferOut) <= 0)
				{
					NSLog(@"VTCompressionSessionEncodeFrame failed");
					return 1;
				}
				*compressedFrameSize = CMSampleBufferGetTotalSampleSize(encoder->sampleBufferOut);
				*allOpaqueAlpha = false;	// don't know how to get this value, doesn't seem to be used for now
				
				// For efficiency, do not copy the data out here, FileWriter will use encoder->sampleBufferOut directly
//				CMBlockBufferRef blockBuffer = CMSampleBufferGetDataBuffer(encoder->sampleBufferOut);
//				status = CMBlockBufferCopyDataBytes(blockBuffer, 0, MIN(*compressedFrameSize, destinationSize), destinationPtr);
			}
		}
		
		if (pixelBuffer)
		{
			CFRelease(pixelBuffer);
		}
	}
	return status;
}

void
PRCloseEncoder(PREncoderRef encoder)
{
	if (encoder->sampleBufferOut)
	{
		CFRelease(encoder->sampleBufferOut);
		encoder->sampleBufferOut = NULL;
	}
	if (encoder->session)
	{
		VTCompressionSessionInvalidate(encoder->session);
		CFRelease(encoder->session);
		encoder->session = NULL;
	}
	delete encoder;
}

PR_EXPORT void PRRelease(PRTypeRef type)
{
	if (ProResFormatDescriptionRef descriptionRef = static_cast<ProResFormatDescriptionRef>(type))
	{
		delete descriptionRef;
	}
	else if (const OpaqueProResFileWriter* writerRef = static_cast<const OpaqueProResFileWriter*>(type))
	{
		delete writerRef;
	}
	else
	{
		NSLog(@"Unimplemented PRRelease for this type %@", CFCopyDescription(type));
	}
}

// matching kPRFrameRate in ProResEncoder.h
constexpr float kFrameRateFromPRFrameRateEnum[12] = {
	0.0f,
	24/1.001f,
	24.0f,
	25.0f,
	30/1.001f,
	30.0f,
	50.0f,
	60/1.001f,
	60.0f,
	100.0f,
	120/1.001f,
	120.0f
};

int
PRSetEncoderProperty(
   PREncoderRef encoder,
   PRPropertyID propID,
   unsigned int propValueSize,
   const void*  propValueAddress)
{
	int propEnum = *(const int*)propValueAddress;
	CFStringRef key;
	CFTypeRef value = NULL;
	bool bNeedRelease = false;
	switch (propID)
	{
		case kPRPropertyID_FrameHeaderFrameRate:
			key = kVTCompressionPropertyKey_ExpectedFrameRate;
			value = CFNumberCreate(kCFAllocatorDefault, kCFNumberFloat32Type, &kFrameRateFromPRFrameRateEnum[propEnum]);
			bNeedRelease = true;
			break;
		case kPRPropertyID_FrameHeaderAspectRatio:
		{
			key = kVTCompressionPropertyKey_PixelAspectRatio;
			CFMutableDictionaryRef valueDict = CFDictionaryCreateMutable(kCFAllocatorDefault,
																		 2,
																		 &kCFTypeDictionaryKeyCallBacks,
																		 &kCFTypeDictionaryValueCallBacks);
			int width = -1;
			int height = -1;
			switch (propEnum) {
				case kPRAspectRatio_Unspecified:
				case kPRAspectRatio_SquarePixel:
				default:
					width = 1;
					height = 1;
					break;
				case kPRAspectRatio_16x9:
					width = 16;
					height = 9;
					break;
				case kPRAspectRatio_4x3:
					width = 4;
					height = 3;
					break;
			}
			CFNumberRef widthRef = CFNumberCreate(kCFAllocatorDefault, kCFNumberIntType, &width);
			CFDictionaryAddValue(valueDict, kCMFormatDescriptionKey_PixelAspectRatioHorizontalSpacing, widthRef);
			CFRelease(widthRef);
			CFNumberRef heightRef = CFNumberCreate(kCFAllocatorDefault, kCFNumberIntType, &height);
			CFDictionaryAddValue(valueDict, kCMFormatDescriptionKey_PixelAspectRatioVerticalSpacing, heightRef);
			CFRelease(heightRef);
			
			value = valueDict;
			bNeedRelease = true;
			
			break;
		}
		case kPRPropertyID_FrameHeaderColorPrimaries:
			key = kVTCompressionPropertyKey_ColorPrimaries;
			switch (propEnum)
			{
				case kPRColorPrimaries_ITU_R_709:
				case kPRColorPrimaries_Unspecified:
				default:
					value = kCMFormatDescriptionColorPrimaries_ITU_R_709_2;
					break;
				case kPRColorPrimaries_EBU_3213:
					value = kCMFormatDescriptionColorPrimaries_EBU_3213;
					break;
				case kPRColorPrimaries_SMPTE_C:
					value = kCMFormatDescriptionColorPrimaries_SMPTE_C;
					break;
				case kPRColorPrimaries_ITU_R_2020:
					value = kCMFormatDescriptionColorPrimaries_ITU_R_2020;
					break;
				case kPRColorPrimaries_DCI_P3:
					value = kCMFormatDescriptionColorPrimaries_DCI_P3;
					break;
				case kPRColorPrimaries_P3_D65:
					value = kCMFormatDescriptionColorPrimaries_P3_D65;
					break;
			}
			break;
		case kPRPropertyID_FrameHeaderTransferCharacteristic:
			key = kVTCompressionPropertyKey_TransferFunction;
			switch (propEnum)
			{
				case kPRTransferCharacteristic_ITU_R_709:
				case kPRTransferCharacteristic_Unspecified:
				default:
					value = kCMFormatDescriptionTransferFunction_ITU_R_709_2;
					break;
				case kPRTransferCharacteristic_ST_2084:
					value = kCMFormatDescriptionTransferFunction_SMPTE_ST_2084_PQ;
					break;
				case kPRTransferCharacteristic_HLG:
					value = kCMFormatDescriptionTransferFunction_ITU_R_2100_HLG;
					break;
			}
			break;
		case kPRPropertyID_FrameHeaderMatrixCoefficients:
			key = kVTCompressionPropertyKey_YCbCrMatrix;
			switch (propEnum)
			{
				case kPRMatrixCoefficients_ITU_R_709:
				case kPRMatrixCoefficients_Unspecified:
				default:
					value = kCMFormatDescriptionYCbCrMatrix_ITU_R_709_2;
					break;
				case kPRMatrixCoefficients_ITU_R_2020:
					value = kCMFormatDescriptionYCbCrMatrix_ITU_R_2020;
					break;
				case kPRMatrixCoefficients_ITU_R_601:
					value = kCMFormatDescriptionYCbCrMatrix_ITU_R_601_4;
					break;
			}
			break;
		default:
			NSLog(@"Unsupported property!");
			return -2195;
	}
	CFDictionaryAddValue(encoder->sessionProperties, key, value);
	
	return noErr;
}

#pragma mark - ProResFileWriter

PR_EXPORT PRStatus ProResFileWriterCreate(
	const char *destUTF8Path,
	ProResFileWriterRef *newAssetWriterOut )
{
	*newAssetWriterOut = new OpaqueProResFileWriter(destUTF8Path);
	return !(*newAssetWriterOut && (*newAssetWriterOut)->innerRef.error == nil);
}

PR_EXPORT PRStatus ProResFileWriterInvalidate(
	ProResFileWriterRef writer )
{
	// nothing to do
	return noErr;
}

PR_EXPORT PRStatus ProResFileWriterSetMovieTimescale(
	ProResFileWriterRef writer,
	PRTimeScale timescale )
{
	writer->innerRef.movieTimeScale = timescale;
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterSetTrackMediaTimescale(
	ProResFileWriterRef writer,
	PRPersistentTrackID writerTrackID,
	PRTimeScale timescale )
{
	AVAssetWriterInput* writerInput = [writer->innerRef.inputs objectAtIndex:writerTrackID];
	// AVFoundation does not allow setting mediaTimeScale with media type AVMediaTypeAudio
	// see AVAssetWriterInput.h:439
	if (writerInput.mediaType != AVMediaTypeAudio)
	{
		writerInput.mediaTimeScale = timescale;
	}
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterSetTrackPreferredChunkSize(
	ProResFileWriterRef writer,
	PRPersistentTrackID writerTrackID,
	int32_t preferredChunkSize )
{
	// Ignore and use default, otherwise will result in some serious padding and bloat up file size sometimes 8x
	//[writer->innerRef.inputs objectAtIndex:writerTrackID].preferredMediaChunkAlignment = preferredChunkSize;
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterAddTrack(
	ProResFileWriterRef writer,
	PRMediaType mediaType,
	PRPersistentTrackID *writerTrackIDOut )
{
	auto input = [[AVAssetWriterInput alloc] initWithMediaType:GetAVMediaTypeFromPRMediaType(mediaType)
												outputSettings:nil];
	input.expectsMediaDataInRealTime = true;
	[writer->innerRef addInput:input];
	*writerTrackIDOut = [writer->innerRef.inputs indexOfObject:input];
	[input release];
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterBeginSession(
	ProResFileWriterRef writer,
	PRTime sessionStartTime )
{
	[writer->innerRef startWriting];
	[writer->innerRef startSessionAtSourceTime:CMTimeFromPRTime(sessionStartTime)];
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterAddSampleBufferToTrack(
	ProResFileWriterRef writer,
	PRPersistentTrackID writerTrackID,
	void *dataBuffer,
	size_t dataBufferLength,
	const PRSampleBufferDeallocator *deallocator,
	ProResFormatDescriptionRef formatDescription,
	int64_t numSamples,
	int64_t numSampleTimingEntries,
	const PRSampleTimingInfo * sampleTimingArray,
	int64_t numSampleSizeEntries,
	const size_t *sampleSizeArray )
{
	AVAssetWriterInput* writerInput = [writer->innerRef.inputs objectAtIndex:writerTrackID];
	if ([writerInput.mediaType isEqualToString:AVMediaTypeVideo])
	{
		NSLog(@"Unimplemented on Mac, use ProResFileWriterAddEncoderSampleBufferToTrack() instead.");
		return 1;
	}
	else if ([writerInput.mediaType isEqualToString:AVMediaTypeTimecode])
	{
		CMBlockBufferRef blockBufferRef;
		int status = CMBlockBufferCreateWithMemoryBlock(kCFAllocatorDefault,
														NULL,
														sizeof(int32_t),
														kCFAllocatorDefault,
														NULL,
														0,
														sizeof(int32_t),
														kCMBlockBufferAssureMemoryNowFlag,
														&blockBufferRef);
		if (status == kCMBlockBufferNoErr && blockBufferRef)
		{
			status = CMBlockBufferReplaceDataBytes(dataBuffer, blockBufferRef, 0, sizeof(int32_t));
			CMSampleTimingInfo sampleTimingArrayCM[numSampleTimingEntries];
			for (int i = 0;	i < numSampleTimingEntries; i++)
			{
				sampleTimingArrayCM[i] = CMSampleTimingInfoFromPRSampleTimingInfo(sampleTimingArray[i]);
			}
			CMSampleBufferRef sampleBufferRef;
			status = CMSampleBufferCreateReady(kCFAllocatorDefault,
											   blockBufferRef,
											   formatDescription->innerRef,
											   numSamples,
											   numSampleTimingEntries,
											   sampleTimingArrayCM,
											   numSampleSizeEntries,
											   sampleSizeArray,
											   &sampleBufferRef);
			if (status == noErr && sampleBufferRef)
			{
				[writerInput appendSampleBuffer:sampleBufferRef];
				CFRelease(sampleBufferRef);
			}
			CFRelease(blockBufferRef);
		}
	}
	else
	{
		NSLog(@"Unsupported media type!");
		return 1;
	}
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterAddEncoderSampleBufferToTrack(
	ProResFileWriterRef writer,
	PRPersistentTrackID writerTrackID,
	PREncoderRef		encoder,
	PRTime				timestamp)
{
	AVAssetWriterInput* writerInput = [writer->innerRef.inputs objectAtIndex:writerTrackID];
	if ([writerInput.mediaType isEqualToString:AVMediaTypeVideo])
	{
		if (encoder->sampleBufferOut && CMSampleBufferDataIsReady(encoder->sampleBufferOut))
		{
			CMSampleBufferSetOutputPresentationTimeStamp(encoder->sampleBufferOut, CMTimeFromPRTime(timestamp));
			[writerInput appendSampleBuffer:encoder->sampleBufferOut];
			CFRelease(encoder->sampleBufferOut);
			encoder->sampleBufferOut = NULL;
		}
		else
		{
			NSLog(@"encoder->sampleBufferOut not ready!");
		}
	}
	else
	{
		NSLog(@"Unsupported media type!");
	}
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterAddAudioSampleBufferToTrack(
	ProResFileWriterRef writer,
	PRPersistentTrackID writerTrackID,
	void *dataBuffer,
	size_t dataBufferLength,
	const PRSampleBufferDeallocator *deallocator,
	ProResFormatDescriptionRef formatDescription,
	int64_t numSamples,
	PRTime timeStamp )
{
	CMBlockBufferRef blockBufferRef;
	int status = CMBlockBufferCreateWithMemoryBlock(kCFAllocatorDefault,
													dataBuffer,
													dataBufferLength,
													kCFAllocatorNull,
													NULL,
													0,
													dataBufferLength,
													0,
													&blockBufferRef);
	if (status == noErr && blockBufferRef)
	{
		CMSampleBufferRef sampleBufferRef;
		status = CMAudioSampleBufferCreateReadyWithPacketDescriptions(kCFAllocatorDefault,
																	  blockBufferRef,
																	  formatDescription->innerRef,
																	  numSamples,
																	  CMTimeFromPRTime(timeStamp),
																	  NULL,
																	  &sampleBufferRef);
		if (status == noErr && sampleBufferRef)
		{
			AVAssetWriterInput* writerInput = [writer->innerRef.inputs objectAtIndex:writerTrackID];
			[writerInput appendSampleBuffer:sampleBufferRef];
			CFRelease(sampleBufferRef);
		}
		CFRelease(blockBufferRef);
	}
	return status;
}

PR_EXPORT PRStatus ProResFileWriterMarkEndOfDataForTrack(
	ProResFileWriterRef writer,
	PRPersistentTrackID writerTrackID )
{
	AVAssetWriterInput* writerInput = [writer->innerRef.inputs objectAtIndex:writerTrackID];
	[writerInput markAsFinished];
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterEndSession(
	ProResFileWriterRef writer,
	PRTime sessionEndTime )
{
	[writer->innerRef endSessionAtSourceTime:CMTimeFromPRTime(sessionEndTime)];
	return writer->innerRef.error != NULL;
}

PR_EXPORT PRStatus ProResFileWriterFinish(
	ProResFileWriterRef writer )
{
	[writer->innerRef finishWritingWithCompletionHandler:^{}];
	return writer->innerRef.error != NULL;
}

#pragma mark - FormatDescription

PR_EXPORT PRStatus ProResTimecodeFormatDescriptionCreate(
	 PRTime frameDuration,
	 uint32_t frameQuanta,
	 uint32_t tcFlags,
	 const char *sourceReferenceName,
	 size_t sourceReferenceNameSize,
	 int16_t languageCode,
	 ProResTimecodeFormatDescriptionRef *outDesc)
{
	CMFormatDescriptionRef ref;
	int status = CMTimeCodeFormatDescriptionCreate(kCFAllocatorDefault,
												   kCMTimeCodeFormatType_TimeCode32,
												   CMTimeFromPRTime(frameDuration),
												   frameQuanta,
												   tcFlags,
												   NULL,
												   &ref);
	if (status == noErr)
	{
		*outDesc = new opaqueFormatDescription(ref);;
	}
	return status;
}

PR_EXPORT PRStatus ProResVideoFormatDescriptionCreate(
	PRVideoCodecType codecType,
	PRVideoDimensions dimensions,
	int32_t depth,
	uint32_t fieldCount,
	ProResFormatDescriptionFieldDetail fieldDetail,
	ProResFormatDescriptionColorPrimaries colorPrimaries,
	ProResFormatDescriptionTransferFunction transferFunction,
	ProResFormatDescriptionYCbCrMatrix matrix,
	uint32_t paspHorizontalSpacing,
	uint32_t paspVerticalSpacing,
	const PRCleanApertureDataRational *clapData,
	bool hasGammaLevel,
	double gammaLevel,
	ProResVideoFormatDescriptionRef *outDesc)
{
	// not needed when sample source is encoded by PREncoder
	return noErr;
}

PR_EXPORT PRStatus ProResAudioFormatDescriptionCreate(
	const AudioStreamBasicDescription *asbd,
	size_t layoutSize,
	const AudioChannelLayout *layout,
	ProResAudioFormatDescriptionRef *outDesc)
{
	CMAudioFormatDescriptionRef ref;
	int status = CMAudioFormatDescriptionCreate(kCFAllocatorDefault,
												asbd,
												layoutSize,
												layout,
												0,
												NULL,
												NULL,
												&ref);
	if (status == noErr)
	{
		*outDesc = new opaqueFormatDescription(ref);
	}
	return status;
}
