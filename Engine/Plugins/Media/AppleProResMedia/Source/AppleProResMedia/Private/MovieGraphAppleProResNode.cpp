// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieGraphAppleProResNode.h"

#include "AppleProResEncoder/AppleProResEncoder.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/MovieGraphOCIOHelper.h"
#include "Graph/Nodes/MovieGraphBurnInNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "ImageWriteTask.h"
#include "MoviePipelineImageQuantization.h"
#include "MoviePipelineTelemetry.h"
#include "MovieRenderPipelineCoreModule.h"
#include "Styling/AppStyle.h"
#include "SampleBuffer.h"

UMovieGraphAppleProResNode::UMovieGraphAppleProResNode()
	: Quality(EAppleProResEncoderCodec::ProRes_422LT)
	, bIncludeAudio(true)
{
	
}

#if WITH_EDITOR
FText UMovieGraphAppleProResNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText AppleProResNodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_AppleProRes", "Apple ProRes Movie");
	return AppleProResNodeName;
}

FText UMovieGraphAppleProResNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "AppleProResNode_Category", "Output Type");
}

FText UMovieGraphAppleProResNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "AppleProResGraphNode_Keywords", "apple pro res prores mov movie video");
	return Keywords;
}

FLinearColor UMovieGraphAppleProResNode::GetNodeTitleColor() const
{
	static const FLinearColor AppleProResNodeColor = FLinearColor(0.047f, 0.654f, 0.537f);
	return AppleProResNodeColor;
}

FSlateIcon UMovieGraphAppleProResNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon AppleProResIcon = FSlateIcon(FName("MovieRenderPipelineStyle"), "MovieRenderPipeline.Graph.Icon.RenderMovieFile");

	OutColor = FLinearColor::White;
	return AppleProResIcon;
}

FText UMovieGraphAppleProResNode::GetBasicConfigShortDisplayName() const
{
	return NSLOCTEXT("MovieGraphNodes", "BasicConfigShortName_PRORES", "PRORES");
}
#endif // WITH_EDITOR

TUniquePtr<MovieRenderGraph::IVideoCodecWriter> UMovieGraphAppleProResNode::Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext)
{
	bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	UMovieGraphGlobalOutputSettingNode* OutputSetting =
		InInitializationContext.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	bIncludeCDOs = false;
	const UMovieGraphAppleProResNode* EvaluatedNode = Cast<UMovieGraphAppleProResNode>(
		InInitializationContext.EvaluatedConfig->GetSettingForBranch(GetClass(), FName(InInitializationContext.PassData->Key.RootBranchName), bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("Apple ProRes node could not be found in the graph in branch [%s]."), *InInitializationContext.PassData->Key.RootBranchName.ToString());
	
	const FFrameRate SourceFrameRate = InInitializationContext.Pipeline->GetDataSourceInstance()->GetDisplayRate();
	const FFrameRate EffectiveFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSetting, SourceFrameRate);
	
	FAppleProResEncoderOptions Options;
	Options.OutputFilename = InInitializationContext.FileName;
	Options.Width = InInitializationContext.Resolution.X;
	Options.Height = InInitializationContext.Resolution.Y;
	Options.FrameRate = EffectiveFrameRate;
	Options.Codec = EvaluatedNode->Quality;
	Options.ColorPrimaries = EAppleProResEncoderColorPrimaries::CD_HDREC709; // Force Rec 709 for now
	Options.ScanMode = EAppleProResEncoderScanMode::IM_PROGRESSIVE_SCAN; // No interlace sources
	Options.bWriteAlpha = true;
	Options.bDropFrameTimecode = OutputSetting->bDropFrameTimecode;
	Options.bIncludeAudio = EvaluatedNode->bIncludeAudio;

	// If OCIO is enabled, don't do additional color conversion
	
	TUniquePtr<FProResWriter> NewWriter = MakeUnique<FProResWriter>();
	NewWriter->Writer = MakeUnique<FAppleProResEncoder>(Options);
	NewWriter->bSkipColorConversions = (EvaluatedNode->bOverride_OCIOConfiguration && EvaluatedNode->OCIOConfiguration.bIsEnabled && InInitializationContext.bAllowOCIO);

	CachedPipeline = InInitializationContext.Pipeline;
	
	return NewWriter;
}

bool UMovieGraphAppleProResNode::Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	const FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);
	if(!CodecWriter->Writer->Initialize())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to initialize Apple ProRes writer.");
		return false;
	}
	
	return true;
}

void UMovieGraphAppleProResNode::WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName)
{
	FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);

	// If the writer was not initialized, don't try to finalize anything.
	if (!CodecWriter->Writer)
	{
		return;
	}

	bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	const UMovieGraphAppleProResNode* EvaluatedNode = Cast<UMovieGraphAppleProResNode>(
		InEvaluatedConfig->GetSettingForBranch(GetClass(), FName(InBranchName), bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("Apple ProRes node could not be found in the graph in branch [%s]."), *InBranchName);

	bIncludeCDOs = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode =
		InEvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);
	
	const UE::MovieGraph::FMovieGraphSampleState* GraphPayload = InPixelData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();
	
	// Translate our Movie Pipeline specific payload to a ProRes Encoder specific payload.
	const TSharedPtr<FAppleProResEncoder::FTimecodePayload, ESPMode::ThreadSafe> ProResPayload = MakeShared<FAppleProResEncoder::FTimecodePayload, ESPMode::ThreadSafe>();

	if (OutputSettingNode->bOverride_CustomTimecodeStart)
	{
		const int32 CustomTimecodeStartFrame = OutputSettingNode->CustomTimecodeStart.ToFrameNumber(CodecWriter->Writer->GetOptions().FrameRate).Value;
		const FMovieGraphTimeStepData& TimeData = GraphPayload->TraversalContext.Time;
		
		// When using a custom timecode start, just use the root-level frame number (relative to zero) offset by the custom timecode start
		ProResPayload->ReferenceFrameNumber = TimeData.OutputFrameNumber + CustomTimecodeStartFrame;
	}
	else
	{
		// This is the frame number on the global time, can have overlaps (between encoders) or repeats when using handle frames/slowmo.
		ProResPayload->ReferenceFrameNumber = GraphPayload->TraversalContext.Time.RootFrameNumber.Value;
	}

	TArray<FPixelPreProcessor> PixelPreProcessors;

	// Run OCIO before quantizing. For highest accuracy, OCIO should run on the non-quantized pixel data.
#if WITH_OCIO
	if (FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessor(GraphPayload, CachedPipeline.Get(), InEvaluatedConfig, EvaluatedNode->OCIOConfiguration, EvaluatedNode->OCIOContext, PixelPreProcessors))
	{
		PixelPreProcessors.Pop()(InPixelData);
	}
#endif

	// ProRes can handle quantization to 8bit internally if necessary; apply sRGB if needed.
	constexpr int32 TargetBitDepth = 16;
	const bool bConvertToSrgb = !CodecWriter->bSkipColorConversions;
	const TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(InPixelData, TargetBitDepth, ProResPayload, bConvertToSrgb);

	// Do a quick composite of renders/burn-ins.
	for (const FMovieGraphPassData& CompositePass : InCompositePasses)
	{
		// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
		// burn in/widget data when we decided to composite it.
		switch (QuantizedPixelData->GetType())
		{
		case EImagePixelType::Color:
			PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float16:
			PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float32:
			PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		}
	}

	// This is done on the current thread for simplicity but the composite itself is parallelized.
	FImagePixelData* PixelData = QuantizedPixelData.Get();
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid.
		PreProcessor(PixelData);
	}

	CodecWriter->Writer->WriteFrame(PixelData);
}

void UMovieGraphAppleProResNode::BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	if (!CachedPipeline.IsValid())
	{
		return;
	}

	const MoviePipeline::FAudioState& AudioData = CachedPipeline->GetAudioRendererInstance()->GetAudioState();
	FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);

	// If the writer was not initialized, don't try to finalize anything.
	if (!CodecWriter->Writer)
	{
		return;
	}

	if (!CodecWriter->Writer->IsInitialized())
	{
		return;
	}

	// Nothing to do here if audio isn't being generated. The "invalid shot index" warning below is legitimate *if audio is being rendered*, but if
	// no audio is being rendered (eg, with -nosound) then we don't want the warning to show up.
	if (!FApp::CanEverRenderAudio())
	{
		return;
	}

	MovieRenderGraph::Audio::WriteShotAudioSamples(
		AudioData,
		CodecWriter->LightweightSourceData,
		CodecWriter->Writer->GetOptions().FrameRate,
		[&](TArrayView<int16> InSamples)
		{
			CodecWriter->Writer->WriteAudioSample(InSamples);
		});
}


void UMovieGraphAppleProResNode::Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	// Write to disk
	const FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);
	CodecWriter->Writer->Finalize();
}

const TCHAR* UMovieGraphAppleProResNode::GetFilenameExtension() const
{
	return TEXT("mov");
}

bool UMovieGraphAppleProResNode::IsAudioSupported() const
{
	return true;
}

void UMovieGraphAppleProResNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesProRes = true;
	InTelemetry->bMovieEncodesAudio |= bIncludeAudio;
}

void UMovieGraphAppleProResNode::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	if (bEnableBurnIn)
	{
		UMovieGraphOutputBurnInNode* BurnInNode = NewObject<UMovieGraphOutputBurnInNode>(InEvaluatedConfig);
		BurnInNode->bOverride_OutputName = true;
		BurnInNode->bOverride_FileNameFormat = true;
		BurnInNode->bOverride_OutputRestriction = true;
		BurnInNode->bOverride_LayerNameFormat = true;
		BurnInNode->bOverride_BurnInClass = true;
		BurnInNode->bOverride_bCompositeOntoFinalImage = true;
		BurnInNode->OutputName = TEXT("ProRes");
		BurnInNode->FileNameFormat = BurnInFileNameFormat;
		BurnInNode->OutputRestriction = FSoftClassPath(GetClass());
		BurnInNode->LayerNameFormat = TEXT("");	// Unused
		BurnInNode->BurnInClass = BurnInClass;
		BurnInNode->bCompositeOntoFinalImage = bCompositeOntoFinalImage;

		OutInjectedNodes.Add(BurnInNode);
	}
}
