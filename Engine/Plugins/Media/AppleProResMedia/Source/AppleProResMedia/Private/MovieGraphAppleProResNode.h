// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AppleProResEncoder/AppleProResEncoder.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/Nodes/MovieGraphVideoOutputNode.h"

#include "MovieGraphAppleProResNode.generated.h"

/** A node which can output Apple ProRes movies. */
UCLASS(BlueprintType)
class UMovieGraphAppleProResNode : public UMovieGraphVideoOutputNode, public IMovieGraphEvaluationNodeInjector
{
	GENERATED_BODY()

public:
	UMovieGraphAppleProResNode();

#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	virtual FText GetMenuCategory() const override;
	virtual FText GetKeywords() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual FText GetBasicConfigShortDisplayName() const override;
#endif

protected:
	// UMovieGraphVideoOutputNode Interface
	virtual TUniquePtr<MovieRenderGraph::IVideoCodecWriter> Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext) override;
	virtual bool Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	virtual void WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName) override;
	virtual void BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	virtual void Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	virtual const TCHAR* GetFilenameExtension() const override;
	virtual bool IsAudioSupported() const override;
	// ~UMovieGraphVideoOutputNode Interface

	// UMovieGraphSettingNode Interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMovieGraphSettingNode Interface

	// IMovieGraphEvaluationNodeInjector Interface
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	// ~IMovieGraphEvaluationNodeInjector Interface

protected:
	struct FProResWriter : public MovieRenderGraph::IVideoCodecWriter
	{
		bool bSkipColorConversions = false;
		TUniquePtr<FAppleProResEncoder> Writer;
	};
	
	/** The pipeline that is running this node. */
	TWeakObjectPtr<UMovieGraphPipeline> CachedPipeline;

public:
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Quality : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bIncludeAudio : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOConfiguration : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOContext : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableBurnIn : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInFileNameFormat : 1;
	
	/** The Apple ProRes codec that should be used. See Apple documentation for more specifics. Uses Rec 709 color primaries. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Apple ProRes", meta = (EditCondition = "bOverride_Quality"))
	EAppleProResEncoderCodec Quality;

	/** If true, audio will be included in the video file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Apple ProRes", meta = (EditCondition = "bOverride_bIncludeAudio"))
	bool bIncludeAudio;

	/**
	* OCIO configuration/transform settings.
	*
	* Note: There are differences from the previous implementation in MRQ given that we are now doing CPU-side processing.
	* 1) This feature only works on desktop platforms when the OpenColorIO library is available.
	* 2) Users are now responsible for setting the renderer output space to Final Color (HDR) in Linear Working Color Space (SCS_FinalColorHDR) by
	*    disabling the Tone Curve setting on the renderer node.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName="OCIO Configuration", meta = (DisplayAfter = "FileNameFormat", EditCondition = "bOverride_OCIOConfiguration"))
	FOpenColorIODisplayConfiguration OCIOConfiguration;

	/**
	* OCIO context of key-value string pairs, typically used to apply shot-specific looks (such as a CDL color correction, or a 1D grade LUT).
	* 
	* Notes:
	* 1) If a configuration asset base context was set, it remains active but can be overridden here with new key-values.
	* 2) Format tokens such as {shot_name} are supported and will get resolved before submission.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName = "OCIO Context", meta = (DisplayAfter = "OCIOConfiguration", EditCondition = "bOverride_OCIOContext"))
	TMap<FString, FString> OCIOContext;

	/** If true, this output node will also generate a burn-in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bEnableBurnIn"))
	bool bEnableBurnIn = false;

	/** The widget that the burn-in should use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Burn In", meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"))
	FSoftClassPath BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));

	/** If true, the burn-in that's generated will be composited onto this output. Otherwise, the burn-in will be written to a different file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bCompositeOntoFinalImage"))
	bool bCompositeOntoFinalImage = true;

	/** The file name format used for writing out the burn-in video (if not composited). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_BurnInFileNameFormat", MrgTokenAutocomplete))
	FString BurnInFileNameFormat = TEXT("{sequence_name}.{layer_name}.{renderer_name}");
};