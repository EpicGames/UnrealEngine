// Copyright Epic Games, Inc. All Rights Reserved.

#include "MoviePipelineAppleProResOutput.h"
#include "MoviePipeline.h"
#include "MoviePipelineOutputSetting.h"
#include "MoviePipelinePrimaryConfig.h"
#include "MoviePipelineTelemetry.h"
#include "ImagePixelData.h"
#include "MoviePipelineImageQuantization.h"
#include "SampleBuffer.h"
#include "MovieRenderPipelineCoreModule.h"
#include "ImageWriteTask.h"
#include "MovieRenderPipelineDataTypes.h"

// For logs
#include "MovieRenderPipelineCoreModule.h"

TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> UMoviePipelineAppleProResOutput::Initialize_GameThread(const FString& InFileName, FIntPoint InResolution, EImagePixelType InPixelType, ERGBFormat InPixelFormat, uint8 InBitDepth, uint8 InNumChannels)
{
	const UMoviePipelineOutputSetting* OutputSettings = GetPipeline()->GetPipelinePrimaryConfig()->FindSetting<UMoviePipelineOutputSetting>();
	if (!OutputSettings)
	{
		return nullptr;
	}
	 
	FAppleProResEncoderOptions Options;
	Options.OutputFilename = InFileName;
	Options.Width = InResolution.X;
	Options.Height = InResolution.Y;
	Options.FrameRate = GetPipeline()->GetPipelinePrimaryConfig()->GetEffectiveFrameRate(GetPipeline()->GetTargetSequence());
	Options.MaxNumberOfEncodingThreads = bOverrideMaximumEncodingThreads ? MaxNumberOfEncodingThreads : 0; // Hardware Determine
	Options.Codec = Codec;
	Options.ColorPrimaries = EAppleProResEncoderColorPrimaries::CD_HDREC709; // Force Rec 709 for now
	Options.ScanMode = EAppleProResEncoderScanMode::IM_PROGRESSIVE_SCAN; // No interlace sources.
	Options.bWriteAlpha = true;
	Options.bIncludeAudio = bIncludeAudio;

	TUniquePtr<FAppleProResEncoder> Encoder = MakeUnique<FAppleProResEncoder>(Options);
	
	TUniquePtr<FProResWriter> OutWriter = MakeUnique<FProResWriter>();
	OutWriter->Writer = MoveTemp(Encoder);
	OutWriter->FileName = InFileName;
	
	return OutWriter;
}

bool UMoviePipelineAppleProResOutput::Initialize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);
	if(!CodecWriter->Writer->Initialize())
	{
		UE_LOGF(LogMovieRenderPipeline, Error, "Failed to initialize Apple Pro Res Writer.");
		return false;
	}

	return true;
}

void UMoviePipelineAppleProResOutput::WriteFrame_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<MoviePipeline::FCompositePassInfo>&& InCompositePasses)
{
	FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);
	FImagePixelDataPayload* PipelinePayload = InPixelData->GetPayload<FImagePixelDataPayload>();
	
	// Translate our Movie Pipeline specific payload to a ProRes Encoder specific payload.
	TSharedPtr<FAppleProResEncoder::FTimecodePayload, ESPMode::ThreadSafe> ProResPayload = MakeShared<FAppleProResEncoder::FTimecodePayload, ESPMode::ThreadSafe>();

	// This is the frame number on the global time, can have overlaps (between encoders) or repeats when using handle frames/slowmo.
	ProResPayload->ReferenceFrameNumber = PipelinePayload->SampleState.OutputState.SourceFrameNumber;

	// ProRes can handle either 16 or 8 bit input internally, but expects Rec709 input which has a sRGB curve applied.
	const bool bConvertTosRGB = InWriter->bConvertToSrgb;
	constexpr int32 TargetBitDepth = 16;
	const TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(InPixelData, TargetBitDepth, ProResPayload, bConvertTosRGB);

	// Do a quick composite of renders/burn-ins.
	TArray<FPixelPreProcessor> PixelPreProcessors;
	for (const MoviePipeline::FCompositePassInfo& CompositePass : InCompositePasses)
	{
		// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
		// burn in/widget data when we decided to composite it.
		switch (QuantizedPixelData->GetType())
		{
		case EImagePixelType::Color:
			PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float16:
			PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float32:
			PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositePass.PixelData->MoveImageDataToNew()));
			break;
		}
	}

	// This is done on the main thread for simplicity but the composite itself is parallaleized.
	FImagePixelData* PixelData = QuantizedPixelData.Get();
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid.
		PreProcessor(PixelData);
	}

	CodecWriter->Writer->WriteFrame(QuantizedPixelData.Get());
}

void UMoviePipelineAppleProResOutput::BeginFinalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	const MoviePipeline::FAudioState& AudioData = GetPipeline()->GetAudioState();
	FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);

	// If the writer was not initialized, don't try to finalize anything.
	if (!CodecWriter->Writer)
	{
		return;
	}

	if (!CodecWriter->Writer->IsInitialized())
	{
		return;
	}
	
	// Nothing to do here if audio isn't being generated. The "invalid shot index" warning below is legitimate *if audio is being rendered*, but if
	// no audio is being rendered (eg, with -nosound) then we don't want the warning to show up.
	if (!FApp::CanEverRenderAudio())
	{
		return;
	}

	for (const TPair<int32, MovieRenderPipeline::IVideoCodecWriter::FLightweightSourceData>& SourceData : CodecWriter->LightweightSourceData)
	{
		if (!AudioData.FinishedSegments.IsValidIndex(SourceData.Key))
		{
			UE_LOGF(LogMovieRenderPipeline, Warning, "Invalid shot index was requested for audio data, skipping audio writes.");
			continue;
		}

		// Look up the audio segment for this shot
		const MoviePipeline::FAudioState::FAudioSegment& AudioSegment = AudioData.FinishedSegments[SourceData.Key];

		// Audio data isn't very sample accurate at this point, so we may have generated slightly more (or less) audio than we expect for
		// the number of frames, so we're actually going to trim down the view of data we provide to match the number of frames rendered,
		// to avoid any excess audio after the end of the video.
		const int32 NumFrames = SourceData.Value.SubmittedFrameCount;

		// ToDo: This is possibly dropping fractions of a sample (ie: 1/48,000th) if the audio sample rate can't be evenly divisible by
		// the frame rate.
		const int32 SamplesPerFrame = AudioSegment.SampleRate * CodecWriter->Writer->GetOptions().FrameRate.AsInterval();
		int32 ExpectedSampleCount = NumFrames * SamplesPerFrame * AudioSegment.NumChannels;

		ExpectedSampleCount = FMath::Min(ExpectedSampleCount, AudioSegment.SegmentData.Num());
		Audio::TSampleBuffer<int16> SampleBuffer = Audio::TSampleBuffer<int16>(AudioSegment.SegmentData.GetData(), ExpectedSampleCount, AudioSegment.NumChannels, AudioSegment.SampleRate);

		const TArrayView<int16> SampleData = SampleBuffer.GetArrayView();
		CodecWriter->Writer->WriteAudioSample(SampleData);
	}
}

void UMoviePipelineAppleProResOutput::Finalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter)
{
	// Commit this to disk.
	FProResWriter* CodecWriter = static_cast<FProResWriter*>(InWriter);
	CodecWriter->Writer->Finalize();
}

#if WITH_EDITOR
FText UMoviePipelineAppleProResOutput::GetDisplayText() const
{
	// When it's called from the CDO it's in the drop down menu so they haven't selected a setting yet.
	if(HasAnyFlags(RF_ArchetypeObject))
	{
		 return NSLOCTEXT("MovieRenderPipeline", "AppleProRes_DisplayNameVariedBits", "Apple ProRes [10-12bit]");
	}
	
	if(Codec == EAppleProResEncoderCodec::ProRes_4444XQ || Codec == EAppleProResEncoderCodec::ProRes_4444)
	{
		return NSLOCTEXT("MovieRenderPipeline", "AppleProRes_DisplayName12Bit", "Apple ProRes [12bit]");
	}
	else
	{
		return NSLOCTEXT("MovieRenderPipeline", "AppleProRes_DisplayName10Bit", "Apple ProRes [10bit]");
	}
}
#endif

void UMoviePipelineAppleProResOutput::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	InTelemetry->bUsesProRes = true;
	InTelemetry->bMovieEncodesAudio |= bIncludeAudio;
}
