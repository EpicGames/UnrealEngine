// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MoviePipelineVideoOutputBase.h"
#include "AppleProResEncoder/AppleProResEncoder.h"
#include "MoviePipelineAppleProResOutput.generated.h"

// Forward Declare
struct FMoviePipelineShotRenderTelemetry;

UCLASS(BlueprintType)
class UMoviePipelineAppleProResOutput : public UMoviePipelineVideoOutputBase
{
	GENERATED_BODY()

		UMoviePipelineAppleProResOutput()
		: UMoviePipelineVideoOutputBase()
		, Codec(EAppleProResEncoderCodec::ProRes_4444XQ)
		, bDropFrameTimecode(false)
		, bIncludeAudio(true)
		, bOverrideMaximumEncodingThreads(false)
		, MaxNumberOfEncodingThreads(0)
	{
	}

protected:
	// UMoviePipelineVideoOutputBase Interface
	virtual TUniquePtr<MovieRenderPipeline::IVideoCodecWriter> Initialize_GameThread(const FString& InFileName, FIntPoint InResolution, EImagePixelType InPixelType, ERGBFormat InPixelFormat, uint8 InBitDepth, uint8 InNumChannels) override;
	virtual bool Initialize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter) override;
	virtual void WriteFrame_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<MoviePipeline::FCompositePassInfo>&& InCompositePasses) override;
	virtual void BeginFinalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter);
	virtual void Finalize_EncodeThread(MovieRenderPipeline::IVideoCodecWriter* InWriter);
	virtual const TCHAR* GetFilenameExtension() const override { return TEXT("mov"); }
	virtual bool IsAudioSupported() const { return true; }
	// ~UMoviePipelineVideoOutputBase Interface

	// UMoviePipelineOutputBase Interface
#if WITH_EDITOR
	virtual FText GetDisplayText() const override;
#endif
	// ~UMoviePipelineOutputBase Interface

	// UMoviePipelineSetting Interface
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	// ~UMoviePipelineSetting Interface

public:
	/** Which Apple ProRes codec should we use? See Apple documentation for more specifics. Uses Rec 709 color primaries. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	EAppleProResEncoderCodec Codec;

	/** Should the embedded timecode track be written using drop-frame format? Only applicable if the sequence framerate is 29.97 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bDropFrameTimecode;

	/** If true, audio will be included in the video file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bIncludeAudio;

	/** Should we override the maximum number of encoding threads? */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Settings")
	bool bOverrideMaximumEncodingThreads;

	/** What is the maximum number of threads the encoder should use to encode frames with? Zero means auto-determine based on hardware. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, meta=(UIMin=0, ClampMin=0, EditCondition="bOverrideMaximumEncodingThreads"), Category = "Settings")
	int32 MaxNumberOfEncodingThreads;

protected:
	struct FProResWriter : public MovieRenderPipeline::IVideoCodecWriter
	{
		TUniquePtr<FAppleProResEncoder> Writer;
	};
};