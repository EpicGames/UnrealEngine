// Copyright Epic Games, Inc. All Rights Reserved.

#ifndef ProResToolboxExtension_h
#define ProResToolboxExtension_h

#include "ProResEncoder.h"
#include "ProResFileWriter.h"

// Implementated on Mac, to reuse internal CMSampleBuffer between PREncoder and ProResFileWriter
PR_EXPORT PRStatus ProResFileWriterAddEncoderSampleBufferToTrack(
	ProResFileWriterRef writer,
	PRPersistentTrackID writerTrackID,
	PREncoderRef		encoder,					/*! @param encoder
														Encoder must have completed a PREncoderFrame, will reuse the last encoded frame.
														Only usable with video track. */
	PRTime				timestamp);					/*! @param timestamp
														OutputPresentationTimeStamp. Use the timestamp in PRSampleTimingInfo.timeStamp. */

#endif /* ProResToolboxExtension_h */
