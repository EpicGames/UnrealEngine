// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfMediaAppleProResDecoder.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "AppleProResMediaModule.h"

#include "AppleProResMediaSettings.h"

#include "GenericPlatform/GenericPlatformAtomics.h"

#include "Windows/AllowWindowsPlatformTypes.h"

const GUID DecoderGUID_AppleProRes_422_Proxy = { 0x6170636F, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_AppleProRes_422_LT = { 0x61706373, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_AppleProRes_422 = { 0x6170636E, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_AppleProRes_422_HQ = { 0x61706368, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_AppleProRes_4444 = { 0x61703468, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };
const GUID DecoderGUID_AppleProRes_4444_XQ = { 0x61703478, 0x767A, 0x494D, { 0xB4, 0x78, 0xF2, 0x9D, 0x25, 0xDC, 0x90, 0x37 } };

bool WmfMediaAppleProResDecoder::IsSupported(const GUID& InGuid)
{
	return ((InGuid == DecoderGUID_AppleProRes_422_Proxy) ||
			(InGuid == DecoderGUID_AppleProRes_422_LT) ||
			(InGuid == DecoderGUID_AppleProRes_422) ||
			(InGuid == DecoderGUID_AppleProRes_422_HQ) ||
			(InGuid == DecoderGUID_AppleProRes_4444) ||
			(InGuid == DecoderGUID_AppleProRes_4444_XQ));
}

bool WmfMediaAppleProResDecoder::SetOutputFormat(const GUID& InGuid, GUID& OutVideoFormat)
{
	if ((InGuid == DecoderGUID_AppleProRes_422_Proxy) ||
		(InGuid == DecoderGUID_AppleProRes_422_LT) ||
		(InGuid == DecoderGUID_AppleProRes_422) ||
		(InGuid == DecoderGUID_AppleProRes_422_HQ) ||
		(InGuid == DecoderGUID_AppleProRes_4444) ||
		(InGuid == DecoderGUID_AppleProRes_4444_XQ))
	{
		OutVideoFormat = MFVideoFormat_ARGB32;
		return true;
	}
	else
	{
		return false;
	}
}

WmfMediaAppleProResDecoder::WmfMediaAppleProResDecoder()
	: WmfMediaDecoder(),
	Decoder(nullptr)
{
}


WmfMediaAppleProResDecoder::~WmfMediaAppleProResDecoder()
{
	if (Decoder)
	{
		PRCloseDecoder(Decoder);
	}
}


#pragma warning(push)
#pragma warning(disable:4838)
HRESULT WmfMediaAppleProResDecoder::QueryInterface(REFIID riid, void** ppv)
{
	static const QITAB qit[] = 
	{
		QITABENT(WmfMediaAppleProResDecoder, IMFTransform),
		{ 0 }
	};

	return QISearch(this, qit, riid, ppv);
}
#pragma warning(pop)


HRESULT WmfMediaAppleProResDecoder::GetOutputAvailableType(DWORD dwOutputStreamID, DWORD dwTypeIndex, IMFMediaType** ppType)
{
	if (ppType == NULL)
	{
		return E_INVALIDARG;
	}

	if (dwOutputStreamID != 0)
	{
		return MF_E_INVALIDSTREAMNUMBER;
	}

	if (dwTypeIndex != 0)
	{
		return MF_E_NO_MORE_TYPES;
	}

	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	TComPtr<IMFMediaType> TempOutputType;

	if (InputType == NULL)
	{
		hr = MF_E_TRANSFORM_TYPE_NOT_SET;
	}

	if (SUCCEEDED(hr))
	{
		hr = MFCreateMediaType(&TempOutputType);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video);
	}

	if (SUCCEEDED(hr))
	{
		TempOutputType->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_Y416);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_FIXED_SIZE_SAMPLES, TRUE);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_ALL_SAMPLES_INDEPENDENT, TRUE);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_SAMPLE_SIZE, OutputImageSize);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFSetAttributeSize(TempOutputType, MF_MT_FRAME_SIZE, ImageWidthInPixels, ImageHeightInPixels);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFSetAttributeRatio(TempOutputType, MF_MT_FRAME_RATE, FrameRate.Numerator, FrameRate.Denominator);
	}

	if (SUCCEEDED(hr))
	{
		hr = TempOutputType->SetUINT32(MF_MT_INTERLACE_MODE, MFVideoInterlace_Progressive);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFSetAttributeRatio(TempOutputType, MF_MT_PIXEL_ASPECT_RATIO, 1, 1);
	}

	if (SUCCEEDED(hr))
	{
		*ppType = TempOutputType;
		(*ppType)->AddRef();
	}

	return hr;
}


HRESULT WmfMediaAppleProResDecoder::ProcessMessage(MFT_MESSAGE_TYPE eMessage, ULONG_PTR ulParam)
{
	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	switch (eMessage)
	{
	case MFT_MESSAGE_COMMAND_FLUSH:
		hr = OnFlush();
		break;

	case MFT_MESSAGE_COMMAND_DRAIN:
	case MFT_MESSAGE_NOTIFY_BEGIN_STREAMING:
		hr = OnDiscontinuity();
		break;

	default:
		break;
	}

	return hr;
}


HRESULT WmfMediaAppleProResDecoder::ProcessOutput(DWORD dwFlags, DWORD cOutputBufferCount, MFT_OUTPUT_DATA_BUFFER* pOutputSamples, DWORD* pdwStatus)
{
	if (dwFlags != 0)
	{
		return E_INVALIDARG;
	}

	if (pOutputSamples == NULL || pdwStatus == NULL)
	{
		return E_POINTER;
	}

	if (cOutputBufferCount != 1)
	{
		return E_INVALIDARG;
	}

	if (OutputQueue.IsEmpty())
	{
		return MF_E_TRANSFORM_NEED_MORE_INPUT;
	}

	FScopeLock Lock(&CriticalSection);

	HRESULT hr = S_OK;

	if (SUCCEEDED(hr))
	{
		TComPtr<IMFSample> Sample;

		MFCreateSample(&Sample);

		pOutputSamples[0].pSample = Sample.Get();
		Sample->AddRef();

		// If used with DX11 we need a (dummy, but amply sized!) memory buffer attached to the sample to make WMF happy
		if (!MediaBuffer.IsValid())
		{
			MFCreateMemoryBuffer(OutputImageSize, &MediaBuffer);
			MediaBuffer->SetCurrentLength(OutputImageSize);
		}

		Sample->AddBuffer(MediaBuffer);
	}

	if (SUCCEEDED(hr))
	{
		hr = InternalProcessOutput(pOutputSamples[0].pSample);
	}

	pOutputSamples[0].dwStatus = 0;
	*pdwStatus = 0;

	return hr;
}

bool WmfMediaAppleProResDecoder::IsExternalBufferSupported() const
{
	return true;
}

HRESULT WmfMediaAppleProResDecoder::InternalProcessOutput(IMFSample* InSample)
{
	if (OutputQueue.IsEmpty())
	{
		return MF_E_TRANSFORM_NEED_MORE_INPUT;
	}

	DataBuffer OuputDataBuffer;
	OutputQueue.Dequeue(OuputDataBuffer);

	LONGLONG TimeStamp = OuputDataBuffer.TimeStamp;

	check(bIsExternalBufferEnabled);

	// Get buffer.
	TArray<uint8>* pExternalBuffer = AllocateExternalBuffer(TimeStamp, OuputDataBuffer.Color.Num());
	if (pExternalBuffer == nullptr)
	{
		return 0;
	}
		
	// Copy to buffer.
	pExternalBuffer->SetNum(OuputDataBuffer.Color.Num());
	FMemory::Memcpy(pExternalBuffer->GetData(), OuputDataBuffer.Color.GetData(), OuputDataBuffer.Color.Num());

	InputQueue.Enqueue(MoveTemp(OuputDataBuffer));

	HRESULT hr = InSample->SetUINT32(MFSampleExtension_CleanPoint, TRUE);

	if (SUCCEEDED(hr))
	{
		hr = InSample->SetSampleTime(TimeStamp);
	}

	if (SUCCEEDED(hr))
	{
		hr = InSample->SetSampleDuration(SampleDuration);
	}

	return hr;
}


HRESULT WmfMediaAppleProResDecoder::OnCheckInputType(IMFMediaType* InMediaType)
{
	HRESULT hr = S_OK;

	if (InputType)
	{
		DWORD dwFlags = 0;
		if (S_OK == InputType->IsEqual(InMediaType, &dwFlags))
		{
			return S_OK;
		}
		else
		{
			return MF_E_INVALIDTYPE;
		}
	}

	GUID majortype = { 0 };
	GUID subtype = { 0 };
	UINT32 width = 0, height = 0;
	MFRatio fps = { 0 };

	hr = InMediaType->GetMajorType(&majortype);

	if (SUCCEEDED(hr))
	{
		if (majortype != MFMediaType_Video)
		{
			hr = MF_E_INVALIDTYPE;
		}
	}

	if (SUCCEEDED(hr))
	{
		hr = InMediaType->GetGUID(MF_MT_SUBTYPE, &subtype);
	}

	if (SUCCEEDED(hr))
	{
		if (!IsSupported(subtype))
		{
			hr = MF_E_INVALIDTYPE;
		}

	}

	if (SUCCEEDED(hr))
	{
		hr = MFGetAttributeSize(InMediaType, MF_MT_FRAME_SIZE, &width, &height);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFGetAttributeRatio(InMediaType, MF_MT_FRAME_RATE, (UINT32*)&fps.Numerator, (UINT32*)&fps.Denominator);
	}

	return hr;
}


HRESULT WmfMediaAppleProResDecoder::OnSetInputType(IMFMediaType* InMediaType)
{
	HRESULT hr = S_OK;

	hr = MFGetAttributeSize(InMediaType, MF_MT_FRAME_SIZE, &ImageWidthInPixels, &ImageHeightInPixels);

	if (SUCCEEDED(hr))
	{
		hr = MFGetAttributeRatio(InMediaType, MF_MT_FRAME_RATE, (UINT32*)&FrameRate.Numerator, (UINT32*)&FrameRate.Denominator);
	}

	if (SUCCEEDED(hr))
	{
		hr = MFFrameRateToAverageTimePerFrame(
			FrameRate.Numerator, 
			FrameRate.Denominator, 
			&SampleDuration
			);
	}

	if (SUCCEEDED(hr))
	{
		InputImageSize = ImageWidthInPixels * ImageHeightInPixels * 8;
		OutputImageSize = ImageWidthInPixels * ImageHeightInPixels * 8; // Y416 (4.4.4 - 16-bit - aligned)

		InputType = InMediaType;
		InputType->AddRef();
	}

	return hr;
}



bool WmfMediaAppleProResDecoder::HasPendingOutput() const
{
	return !InputQueue.IsEmpty();
}


HRESULT WmfMediaAppleProResDecoder::InternalProcessInput(LONGLONG InTimeStamp, BYTE* InData, DWORD InDataSize)
{
	check(InData)

	DataBuffer InputDataBuffer;

	if (!InputQueue.IsEmpty())
	{
		InputQueue.Dequeue(InputDataBuffer);
	}
	else
	{
		InputDataBuffer.Color.AddUninitialized(InputImageSize);
	}

	if (Decoder == nullptr)
	{
		Decoder = PROpenDecoder(GetDefault<UAppleProResMediaSettings>()->NumberOfCPUDecodingThreads, nullptr);
	}

	if (Decoder)
	{
		PRPixelBuffer DestinationBuffer;
		DestinationBuffer.baseAddr = InputDataBuffer.Color.GetData();
		DestinationBuffer.format = kPRFormat_y416;
		DestinationBuffer.width = ImageWidthInPixels;
		DestinationBuffer.height = ImageHeightInPixels;
		DestinationBuffer.rowBytes = ImageWidthInPixels * 8;

		int DecodedBytes = PRDecodeFrame(Decoder, InData, InDataSize, &DestinationBuffer, kPRFullSize, false);

		if (DecodedBytes >= 0)
		{
			InputDataBuffer.TimeStamp = InTimeStamp;
			OutputQueue.Enqueue(MoveTemp(InputDataBuffer));
		}
	}
	return S_OK;
}

#include "Windows/HideWindowsPlatformTypes.h"

#endif // WMFMEDIA_SUPPORTED_PLATFORM
