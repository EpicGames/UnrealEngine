// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "WmfMediaCodec/WmfMediaDecoder.h"

#if WMFMEDIA_SUPPORTED_PLATFORM

#include "ProResDecoder.h"

struct ID3D11Texture2D;

class WmfMediaAppleProResDecoder 
	: public WmfMediaDecoder
{

public:

	WmfMediaAppleProResDecoder();
	virtual ~WmfMediaAppleProResDecoder();

	virtual HRESULT STDMETHODCALLTYPE QueryInterface(REFIID iid, void** ppv) override;
	virtual HRESULT STDMETHODCALLTYPE GetOutputAvailableType(DWORD dwOutputStreamID, DWORD dwTypeIndex, IMFMediaType** ppType) override;
	virtual HRESULT STDMETHODCALLTYPE ProcessMessage(MFT_MESSAGE_TYPE eMessage, ULONG_PTR ulParam) override;
	virtual HRESULT STDMETHODCALLTYPE ProcessOutput(DWORD dwFlags, DWORD cOutputBufferCount, MFT_OUTPUT_DATA_BUFFER* pOutputSamples, DWORD* pdwStatus) override;

	virtual bool IsExternalBufferSupported() const override;

public:

	static bool IsSupported(const GUID& InGuid);
	static bool SetOutputFormat(const GUID& InGuid, GUID& OutVideoFormat);

private:

	HRESULT InternalProcessOutput(IMFSample *InSample);
	virtual HRESULT InternalProcessInput(LONGLONG InTimeStamp, BYTE* InData, DWORD InDataSize) override;
	virtual HRESULT OnCheckInputType(IMFMediaType *InMediaType) override;
	virtual HRESULT OnSetInputType(IMFMediaType *InMediaType) override;

	virtual bool HasPendingOutput() const override;

protected:
	
	PRDecoderRef Decoder;
	TComPtr<IMFMediaBuffer> MediaBuffer;
};

#endif // WMFMEDIA_SUPPORTED_PLATFORM
