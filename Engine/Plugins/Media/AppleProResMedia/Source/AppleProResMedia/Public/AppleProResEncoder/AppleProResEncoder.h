// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/FrameRate.h"
#include "ImagePixelData.h"

// THIRDPARTY_INCLUDES_START
#define __LLP64__ 1
#include "ProResProperties.h"
#include "ProResEncoder.h"
#include "ProResFileWriter.h"
// THIRDPARTY_INCLUDES_END

#include "AppleProResEncoder.generated.h"

#define UE_API APPLEPRORESMEDIA_API

UENUM(BlueprintType)
enum class EAppleProResEncoderCodec : uint8
{
	/** Highest Compression. Approximately 45mbps @ 1920x1080@30fps */
	ProRes_422Proxy UMETA(DisplayName = "Apple ProRes 422 Proxy"),

	/** High compression. Approximately 100mbps @ 1920x1080@30fps */
	ProRes_422LT UMETA(DisplayName = "Apple ProRes 422 LT"),

	/** High Quality Compression for 422 RGB Sources. Approximately 150mbps @ 1920x1080@30fps*/
	ProRes_422 UMETA(DisplayName = "Apple ProRes 422"),

	/** A higher bit-rate version of Apple ProRes 422. Approximately 225mbps @ 1920x1080@30fps*/
	ProRes_422HQ UMETA(DisplayName = "Apple ProRes 422 HQ"),

	/** Extremely high quality and supports alpha channels. Can support both RGB and YCbCr formats. Very large file size. Approximately 330mbps @ 1920x1080@30fps */
	ProRes_4444 UMETA(DisplayName = "Apple ProRes 4444"),

	/** Highest quality storage with support for alpha channel with up to 12 bits precision for RGB and 16 bits for Alpha. Extremely large file size. Approximately 500mbps @ 1920x1080@30fps */
	ProRes_4444XQ UMETA(DisplayName = "Apple ProRes 4444 XQ"),
};

UENUM(BlueprintType)
enum class EAppleProResEncoderColorPrimaries : uint8
{
	CD_SDREC601_525_60HZ UMETA(DisplayName = "SD Rec. 601 525/60Hz"),
	CD_SDREC601_625_50HZ UMETA(DisplayName = "SD Rec. 601 625/50Hz"),
	CD_HDREC709 UMETA(DisplayName = "HD Rec. 709"),
};

UENUM(BlueprintType)
enum class EAppleProResEncoderScanMode : uint8
{
	IM_PROGRESSIVE_SCAN UMETA(DisplayName = "Progressive"),
	IM_INTERLACED_TOP_FIELD_FIRST UMETA(DisplayName = "Interlaced; first (top) image line belongs to first temporal field"),
	IM_INTERLATED_BOTTOM_FIRST_FIRST UMETA(DisplayName = "Interlaced; second (bottom) image line belongs to first temporal field"),
};


struct FAppleProResEncoderOptions
{
	// Disable deprecation warnings for bConvertToSrgb to make Clang happy.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAppleProResEncoderOptions()
		: OutputFilename()
		, Width(0)
		, Height(0)
		, FrameRate(30, 1)
		, Codec(EAppleProResEncoderCodec::ProRes_422)
		, ColorPrimaries(EAppleProResEncoderColorPrimaries::CD_HDREC709)
		, bConvertToSrgb(true)
		, ScanMode(EAppleProResEncoderScanMode::IM_PROGRESSIVE_SCAN)
		, MaxNumberOfEncodingThreads(0)
		, bWriteAlpha(false)
		, bDropFrameTimecode(false)
		, FrameNumberOffset(0)
		, bIncludeAudio(true)
		, AudioSampleRate(48000)
		, AudioChannelCount(2)
	{}
	
	FAppleProResEncoderOptions(FAppleProResEncoderOptions&) = default;
	~FAppleProResEncoderOptions() = default;
	FAppleProResEncoderOptions(const FAppleProResEncoderOptions&) = default;
	FAppleProResEncoderOptions(FAppleProResEncoderOptions&&) = default;
	FAppleProResEncoderOptions& operator=(const FAppleProResEncoderOptions&) = default;
	FAppleProResEncoderOptions& operator=(FAppleProResEncoderOptions&&) = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** The absolute path on disk to try and save the video file to. */
	FString OutputFilename;

	/** The width of the video file. */
	uint32 Width;

	/** The height of the video file. */
	uint32 Height;

	/** Frame Rate of the output video. */
	FFrameRate FrameRate;

	/** Which ProRes codec should we use? Not all support alpha channels. */
	EAppleProResEncoderCodec Codec;

	/** Which color primaries do we use? Only Rec 709 is well tested right now. */
	EAppleProResEncoderColorPrimaries ColorPrimaries;

	/** Whether the data should be converted to sRGB before being sent to the encoder. Should not be used when OCIO is active. */
	UE_DEPRECATED(5.6, "The encoder currently always assumes Rec 709 input, and never used this parameter.")
	bool bConvertToSrgb;

	/** Which scan mode do we use? Only Progressive is tested right now. */
	EAppleProResEncoderScanMode ScanMode;
	
	/** Maximum number of threads to use for encoding. Set to 0 for auto-determine based on hardware. */
	int32 MaxNumberOfEncodingThreads;

	/** If true, attempts to write the alpha channel from the incoming pixel data. Increases encoding time. Only works on some Codecs. */
	bool bWriteAlpha;

	/** If true, timecode track will use drop frame notation for the 29.97 frame rate. */
	bool bDropFrameTimecode;

	/** The number of frames to offset the Timecode track by. */
	int32 FrameNumberOffset;

	/** If false, then audio tracks will not be written and calls to WriteAudioSample will be ignored. */
	bool bIncludeAudio;

	/** The sample rate of the incoming audio data. Only tested with 48'000. */
	uint32 AudioSampleRate;

	/** The number of audio channels in the encoding data. Only tested with 2. */
	uint32 AudioChannelCount;
};

class FAppleProResEncoder
{
public:
	struct FTimecodePayload : IImagePixelDataPayload
	{
		int32 ReferenceFrameNumber;
	};

public:
	UE_API FAppleProResEncoder(const FAppleProResEncoderOptions& InOptions);
	UE_API ~FAppleProResEncoder();

	/** Call to initialize the Sink Writer. This must be done before attempting to write data to it. */
	UE_API bool Initialize();

	/** Finalize the video file and finish writing it to disk. Called by the destructor if not automatically called. */
	UE_API void Finalize();

	/** Appends a new frame onto the output file. */
	UE_API bool WriteFrame(const FImagePixelData* InPixelData);
	
	/** Appends a new audio sample onto the audio stream. */
	UE_API bool WriteAudioSample(const TArrayView<int16>& InAudioSamples);

	/** Gets the options that the encoder was initialized with. */
	UE_API const FAppleProResEncoderOptions& GetOptions() const;

	bool IsInitialized() const { return bInitialized; }

private:
	UE_API bool InitializeVideoTrack();
	UE_API bool InitializeTimecodeTrack();
	UE_API bool InitializeAudioTrack();
	UE_API void ReleaseAndClearResources();

private:
	FAppleProResEncoderOptions Options;
	bool bInitialized;
	bool bFinalized;

	PRPersistentTrackID VideoTrackId;
	PRPersistentTrackID AudioTrackId;
	PRPersistentTrackID TimecodeTrackId;

	/** If valid, we own the memory and need to free it. */
	ProResVideoFormatDescriptionRef VideoFormatDescription;
	/** If valid, we own the memory and need to free it. */
	ProResAudioFormatDescriptionRef AudioFormatDescription;
	/** If valid, we own the memory and need to free it. */
	ProResTimecodeFormatDescriptionRef TimecodeFormatDescription;
	/** If valid, we own the memory and need to free it. */
	ProResFileWriterRef FileWriter;
	/** If valid, we own the memory and need to free it. */
	PREncoderRef Encoder;

	/** Time in the file for video tracks. Incremented by 1 frame for each new video sample. Video samples come in one at a time so this keeps track of how many have been added. */
	PRTime CurrentVideoTime;
	/** Time in the file for audio tracks. Uses a different timescale than Video, as it's in AudioSampleRate. */
	PRTime CurrentAudioTime;


	int32 MaxCompressedFrameSize;
	int32 TargetCompressedFrameSize;
	uint32 AudioSamplesPerFrame;
};

#undef UE_API
